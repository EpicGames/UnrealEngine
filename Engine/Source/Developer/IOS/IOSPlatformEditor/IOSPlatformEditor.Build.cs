// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IOSPlatformEditor : ModuleRules
{
	public IOSPlatformEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "IOS";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"DesktopPlatform",
				"Engine",
				"MainFrame",
				"Slate",
				"SlateCore",
				
				"PropertyEditor",
				"SharedSettingsWidgets",
				"SourceControl",
				"IOSRuntimeSettings",
				"TargetPlatform",
				"MaterialShaderQualitySettings",
				"RenderCore",
                "AudioSettingsEditor",
                "GameProjectGeneration",
				"FreeImage",
                "MacTargetPlatformSettings",
				"MacTargetPlatformControls",
				"ShaderPlatformConfigEditor",
				"RHI"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"GameProjectGeneration",
				"Settings",
				"TurnkeySupport",
			}
		);
	}
}
