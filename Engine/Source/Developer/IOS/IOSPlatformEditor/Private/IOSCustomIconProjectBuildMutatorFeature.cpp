// Copyright Epic Games, Inc. All Rights Reserved.
#include "IOSCustomIconProjectBuildMutatorFeature.h"
#include "CoreMinimal.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "PlatformInfo.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/CoreMisc.h"
#include "Interfaces/ITurnkeySupportModule.h"

static bool RequiresBuild()
{
	// determine if there are any project icons
	FString IconDir = FPaths::Combine(FPaths::ProjectDir(), TEXT("Build/IOS/Resources/Graphics"));
	struct FDirectoryVisitor : public IPlatformFile::FDirectoryVisitor
	{
		TArray<FString>& FileNames;

		FDirectoryVisitor(TArray<FString>& InFileNames)
			: FileNames(InFileNames)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			FString FileName(FilenameOrDirectory);
			if (FileName.EndsWith(TEXT(".png")) && FileName.Contains(TEXT("Icon")))
			{
				FileNames.Add(FileName);
			}
			return true;
		}
	};

	// Enumerate the contents of the current directory
	TArray<FString> FileNames;
	FDirectoryVisitor Visitor(FileNames);
	FPlatformFileManager::Get().GetPlatformFile().IterateDirectory(*IconDir, Visitor);

	if (FileNames.Num() > 0)
	{
		return true;
	}
	return false;
}

bool FIOSCustomIconProjectBuildMutatorFeature ::RequiresProjectBuild(const FName& InPlatformInfoName, FText& OutReason) const
{
#if UE_WITH_TURNKEY_SUPPORT
	const PlatformInfo::FTargetPlatformInfo* const PlatInfo = PlatformInfo::FindPlatformInfo(InPlatformInfoName);
	check(PlatInfo);

	if (ITurnkeySupportModule::Get().GetSdkInfo(PlatInfo->IniPlatformName).Status == ETurnkeyPlatformSdkStatus::Valid)
	{
		const ITargetPlatform* const Platform = GetTargetPlatformManager()->FindTargetPlatform(PlatInfo->Name.ToString());
		if (Platform)
		{
			if (InPlatformInfoName.ToString() == TEXT("IOS"))
			{
				bool bResult = RequiresBuild();
				if (bResult)
				{
					OutReason = NSLOCTEXT("IOSPlatformEditor", "RequiresBuildDueToCustomIcon", "custom icon for IOS");
				}
				return bResult;
			}
		}
	}
#endif // UE_WITH_TURNKEY_SUPPORT
	return false;
}