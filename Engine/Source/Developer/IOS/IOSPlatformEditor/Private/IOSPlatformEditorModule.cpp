// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSPlatformEditorModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "PropertyEditorModule.h"
#include "RenderingThread.h"
#include "IOSRuntimeSettings.h"
#include "IOSTargetSettingsCustomization.h"
#include "ISettingsModule.h"
#include "MaterialShaderQualitySettings.h"
#include "MaterialShaderQualitySettingsCustomization.h"
#include "ComponentRecreateRenderStateContext.h"
#include "ShaderPlatformQualitySettings.h"
#include "Features/IModularFeatures.h"
#include "ISettingsSection.h"

#define LOCTEXT_NAMESPACE "FIOSPlatformEditorModule"

FSimpleMulticastDelegate FIOSPlatformEditorModule::OnSelect;

void FIOSPlatformEditorModule::StartupModule()
{
	// register settings detail panel customization
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(
		"IOSRuntimeSettings",
		FOnGetDetailCustomizationInstance::CreateStatic(&FIOSTargetSettingsCustomization::MakeInstance)
	);

	FOnUpdateMaterialShaderQuality UpdateMaterials = FOnUpdateMaterialShaderQuality::CreateLambda([]()
	{
		FGlobalComponentRecreateRenderStateContext Recreate;
		FlushRenderingCommands();
		UMaterial::AllMaterialsCacheResourceShadersForRendering();
		UMaterialInstance::AllMaterialsCacheResourceShadersForRendering();
	});

	PropertyModule.RegisterCustomClassLayout(
		UShaderPlatformQualitySettings::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialShaderQualitySettingsCustomization::MakeInstance, UpdateMaterials)
		);

	PropertyModule.NotifyCustomizationModuleChanged();

	// register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		TSharedPtr<ISettingsSection> SelectedSection = SettingsModule->RegisterSettings("Project", "Platforms", "iOS",
			LOCTEXT("RuntimeSettingsName", "iOS"),
			LOCTEXT("RuntimeSettingsDescription", "Settings and resources for the iOS platform"),
			GetMutableDefault<UIOSRuntimeSettings>()
		);

		SelectedSection->OnSelect().BindRaw(this, &FIOSPlatformEditorModule::HandleSelectIOSSection);

		{
			static FName NAME_SF_METAL_ES3_1_IOS(TEXT("SF_METAL_ES3_1_IOS"));
			UShaderPlatformQualitySettings* IOSMaterialQualitySettings = UMaterialShaderQualitySettings::Get()->GetShaderPlatformQualitySettings(NAME_SF_METAL_ES3_1_IOS);
			SettingsModule->RegisterSettings("Project", "Platforms", "iOSMetalQuality",
				LOCTEXT("IOSMetalQualitySettingsName", "iOS Material Quality"),
				LOCTEXT("IOSMetalQualitySettingsDescription", "Settings for iOS material quality"),
				IOSMaterialQualitySettings
			);
		}
	}

	IModularFeatures::Get().RegisterModularFeature(PROJECT_BUILD_MUTATOR_FEATURE, &ProjectBuildMutator);
}

void FIOSPlatformEditorModule::ShutdownModule()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Platforms", "iOS");
		SettingsModule->UnregisterSettings("Project", "Platforms", "iOSMetalQuality");
	}

	IModularFeatures::Get().UnregisterModularFeature(PROJECT_BUILD_MUTATOR_FEATURE, &ProjectBuildMutator);
}

void FIOSPlatformEditorModule::HandleSelectIOSSection()
{
	FIOSPlatformEditorModule::OnSelect.Broadcast();
}

IMPLEMENT_MODULE(FIOSPlatformEditorModule, IOSPlatformEditor);

#undef LOCTEXT_NAMESPACE
