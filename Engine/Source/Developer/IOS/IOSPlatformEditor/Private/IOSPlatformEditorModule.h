// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "IOSCustomIconProjectBuildMutatorFeature.h"

/**
 * Module for iOS as a target platform
 */
class FIOSPlatformEditorModule : public IModuleInterface
{
	// IModuleInterface interface
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	virtual void HandleSelectIOSSection();

	// Delegate to notify interested parties when the client sources have changed
	static FSimpleMulticastDelegate OnSelect;

private:
	FIOSCustomIconProjectBuildMutatorFeature ProjectBuildMutator;
};

