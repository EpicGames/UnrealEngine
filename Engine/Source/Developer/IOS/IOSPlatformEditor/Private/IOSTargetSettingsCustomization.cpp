// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSTargetSettingsCustomization.h"
#include "Containers/Ticker.h"
#include "Misc/App.h"
#include "Misc/MonitoredProcess.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Styling/AppStyle.h"
#include "IOSRuntimeSettings.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailGroup.h"
#include "SExternalImageReference.h"
#include "PlatformIconInfo.h"
#include "SourceControlHelpers.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Views/SListView.h"
#include "SProvisionListRow.h"
#include "SCertificateListRow.h"
#include "Misc/EngineBuildSettings.h"
#include "Widgets/Input/SNumericDropDown.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "IOSPlatformEditorModule.h"
#include "HAL/PlatformFileManager.h"
#include "UEFreeImage.h"
#include "XcodeProjectSettings.h"
#include "ShaderPlatformConfigEditor.h"

#define LOCTEXT_NAMESPACE "IOSTargetSettings"
DEFINE_LOG_CATEGORY_STATIC(LogIOSTargetSettings, Log, All);

bool SProvisionListRow::bInitialized = false;
FCheckBoxStyle SProvisionListRow::ProvisionCheckBoxStyle;

const FString gProjectNameText("[PROJECT_NAME]");

//////////////////////////////////////////////////////////////////////////
// FIOSTargetSettingsCustomization
namespace FIOSTargetSettingsCustomizationConstants
{
	const FText DisabledTip = LOCTEXT("GitHubSourceRequiredToolTip", "This requires GitHub source.");
}


TSharedRef<IDetailCustomization> FIOSTargetSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FIOSTargetSettingsCustomization);
}

FIOSTargetSettingsCustomization::FIOSTargetSettingsCustomization()
	: EngineInfoPath(FString::Printf(TEXT("%sBuild/IOS/UnrealGame-Info.plist"), *FPaths::EngineDir()))
	, GameInfoPath(FString::Printf(TEXT("%sBuild/IOS/Info.plist"), *FPaths::ProjectDir()))
	, EngineGraphicsPath(FString::Printf(TEXT("%sBuild/IOS/Resources/Graphics"), *FPaths::EngineDir()))
	, GameGraphicsPath(FString::Printf(TEXT("%sBuild/IOS/Resources/Graphics"), *FPaths::ProjectDir()))
	, TVOSEngineGraphicsPath(FString::Printf(TEXT("%sBuild/TVOS/Resources/Graphics"), *FPaths::EngineDir()))
	, TVOSGameGraphicsPath(FString::Printf(TEXT("%sBuild/TVOS/Resources/Graphics"), *FPaths::ProjectDir()))

{

    // Default AppIcons copied at the payload's root. See https://developer.apple.com/design/human-interface-guidelines/ios/icons-and-images/app-icon/
   
	new (IconNames)FPlatformIconInfo(TEXT("Icon1024.png"), LOCTEXT("AppIcon_Marketing", "Marketing Icon (1024x1024)\n\nOther iOS icons sizes can be generated from the Marketing Icon."), FText::GetEmpty(), 1024, 1024, FPlatformIconInfo::Required); // App Store

	new (IconNames) FPlatformIconInfo(TEXT("Icon60@2x.png"), LOCTEXT("Default_iPhone_AppIcon", "Default iPhone Icon (120x120)"), FText::GetEmpty(), 120, 120, FPlatformIconInfo::Required); // iPhone
    new (IconNames) FPlatformIconInfo(TEXT("Icon76@2x.png"), LOCTEXT("Default_iPad_AppIcon", "Default iPad App Icon (152x152)"), FText::GetEmpty(), 152, 152, FPlatformIconInfo::Required); // iPad, iPad Mini
 
    //From here, all the icons are part of the asset catalog (Assets.car)

    // Required in the asset catalog
    new (IconNames)FPlatformIconInfo(TEXT("Icon83.5@2x.png"), LOCTEXT("iPad_Pro_Retina_App_Icon", "iPad Pro Retina App Icon (167x167)"), FText::GetEmpty(), 167, 167, FPlatformIconInfo::Required); // iPad Pro

    // From here icons are optional
    new (IconNames)FPlatformIconInfo(TEXT("Icon60@3x.png"), LOCTEXT("3x_iPhone_App_Icon", "3x iPhone App Icon (180x180)"), FText::GetEmpty(), 180, 180, FPlatformIconInfo::Optional); // iPhone

    new (IconNames) FPlatformIconInfo(TEXT("Icon40@3x.png"), LOCTEXT("3x_iPhone_Spotlight_Icon", "3x iPhone Spotlight Icon (120x120)"), FText::GetEmpty(), 120, 120, FPlatformIconInfo::Optional); // iPhone
    new (IconNames) FPlatformIconInfo(TEXT("Icon40@2x.png"), LOCTEXT("Default_Spotlight_Icon", "Default Spotlight Icon (80x80)"), FText::GetEmpty(), 80, 80, FPlatformIconInfo::Optional); // iPhone, iPad Pro, iPad, iPad Mini

    new (IconNames) FPlatformIconInfo(TEXT("Icon29@3x.png"), LOCTEXT("3x_iPhone_Settings_Icon", "3x iPhone Settings Icon (87x87)"), FText::GetEmpty(), 87, 87, FPlatformIconInfo::Optional); // iPhone
    new (IconNames) FPlatformIconInfo(TEXT("Icon29@2x.png"), LOCTEXT("Default_Settings_Icon", "Default Settings Icon (58x58)"), FText::GetEmpty(), 58, 58, FPlatformIconInfo::Optional); // iPhone, iPad Pro, iPad, iPad Mini

    new (IconNames) FPlatformIconInfo(TEXT("Icon20@3x.png"), LOCTEXT("3x_iPhone_Notification_Icon", "3x iPhone Notification Icon (60x60)"), FText::GetEmpty(), 60, 60, FPlatformIconInfo::Optional); // iPhone
    new (IconNames) FPlatformIconInfo(TEXT("Icon20@2x.png"), LOCTEXT("Default_Notification_Icon", "Default Notification Icon (40x40)"), FText::GetEmpty(), 40, 40, FPlatformIconInfo::Optional); // iPhone, iPad Pro, iPad, iPad Mini

	// LaunchScreen iOS and tvOS
	new (LaunchImageNames)FPlatformIconInfo(TEXT("LaunchScreenIOS.png"), LOCTEXT("LaunchImageIOS", "Launch Screen Image"), LOCTEXT("LaunchImageIOSDesc", "This image is used for the Launch Screen when custom launch screen storyboards are not in use. The image is used in both portait and landscape modes and will be uniformly scaled to occupy the full width or height as necessary for of all devices, so if your app supports both a square image is recommended. The png file supplied must not have an alpha channel."), -1, -1, FPlatformIconInfo::Required);

	// Icons and Shelf Images for tvOS

    // Used to generate top shelf images
    new (TvOSImageNames)FPlatformIconInfo(TEXT("TopShelfWide-2320x720@2x.png"), LOCTEXT("2x_TVOS_Top_Shelf_Wide", "2x Top Shelf Wide (4640x1440)\n\nOther tvOS Topshelf Image sizes can be generated from it."), FText::GetEmpty(), 4640, 1440, FPlatformIconInfo::Required);

    // Generated top shelf image
    new (TvOSImageNames)FPlatformIconInfo(TEXT("TopShelfWide-2320x720.png"), LOCTEXT("TVOS_Top_Shelf_Wide", "Top Shelf Wide (2320x720)"), FText::GetEmpty(), 2320, 720, FPlatformIconInfo::Optional);

    // Used to generate other tvOS icons
    new (TvOSImageNames)FPlatformIconInfo(TEXT("Icon_Large_Front.png"), LOCTEXT("TVOS_Icon_Large_Front", "Icon Large Front (1280x768)\n\nOther tvOS icons sizes can be generated from it."), FText::GetEmpty(), 1280, 768, FPlatformIconInfo::Required);

    // Generated icons
    new (TvOSImageNames)FPlatformIconInfo(TEXT("Icon_Large_Middle.png"), LOCTEXT("TVOS_Icon_Large_Middle", "Icon Large Middle (1280x768)"), FText::GetEmpty(), 1280, 768, FPlatformIconInfo::Required);
    new (TvOSImageNames)FPlatformIconInfo(TEXT("Icon_Large_Back.png"), LOCTEXT("TVOS_Icon_Large_Back", "Icon Large Back (1280x768)"), FText::GetEmpty(), 1280, 768, FPlatformIconInfo::Required);
    
    new (TvOSImageNames)FPlatformIconInfo(TEXT("Icon_Medium_Front.png"), LOCTEXT("TVOS_Icon_Medium_Front", "Icon Medium Front (800x480)"), FText::GetEmpty(), 800, 480, FPlatformIconInfo::Required);
    new (TvOSImageNames)FPlatformIconInfo(TEXT("Icon_Medium_Middle.png"), LOCTEXT("TVOS_Icon_Medium_Middle", "Icon Medium Middle (800x480)"), FText::GetEmpty(), 800, 480, FPlatformIconInfo::Required);
    new (TvOSImageNames)FPlatformIconInfo(TEXT("Icon_Medium_Back.png"), LOCTEXT("TVOS_Icon_Medium_Back", "Icon Medium Back (800x480)"), FText::GetEmpty(), 800, 480, FPlatformIconInfo::Required);

    new (TvOSImageNames)FPlatformIconInfo(TEXT("Icon_Small_Front.png"), LOCTEXT("TVOS_Icon_Small_Front", "Icon Small Front (400x240)"), FText::GetEmpty(), 400, 240, FPlatformIconInfo::Optional);
    new (TvOSImageNames)FPlatformIconInfo(TEXT("Icon_Small_Middle.png"), LOCTEXT("TVOS_Icon_Small_Middle", "Icon Small Middle (400x240)"), FText::GetEmpty(), 400, 240, FPlatformIconInfo::Optional);
    new (TvOSImageNames)FPlatformIconInfo(TEXT("Icon_Small_Back.png"), LOCTEXT("TVOS_Icon_Small_Back", "Icon Small Back (400x240)"), FText::GetEmpty(), 400, 240, FPlatformIconInfo::Optional);

	bShowAllProvisions = false;
	bShowAllCertificates = false;
	ProvisionList = MakeShareable(new TArray<ProvisionPtr>());
	CertificateList = MakeShareable(new TArray<CertificatePtr>());

	FIOSPlatformEditorModule::OnSelect.AddRaw(this, &FIOSTargetSettingsCustomization::OnSelect);

}

FIOSTargetSettingsCustomization::~FIOSTargetSettingsCustomization()
{
	if (IPPProcess.IsValid())
	{
		IPPProcess = NULL;
		FTSTicker::RemoveTicker(TickerHandle);
	}

	FIOSPlatformEditorModule::OnSelect.RemoveAll(this);
}

void FIOSTargetSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	SavedLayoutBuilder = &DetailLayout;

	BuildPListSection(DetailLayout);

	BuildIconSection(DetailLayout);

	BuildRemoteBuildingSection(DetailLayout);
    
    BuildSecondaryRemoteMacBuildingSection(DetailLayout);

	AudioPluginWidgetManager.BuildAudioCategory(DetailLayout, FString(TEXT("IOS")));

	UE::ShaderPlatformConfigEditor::AddShaderPlatformConfigsFromSettingsModule(DetailLayout, "IOSTargetPlatformSettings");
}

void FIOSTargetSettingsCustomization::OnSelect()
{
	FindRequiredFiles();
}

static FString OutputMessage;
static void OnOutput(FString Message)
{
	OutputMessage += Message;
	OutputMessage += "\n";
	UE_LOGF(LogTemp, Log, "%ls", *Message);
}

void FIOSTargetSettingsCustomization::UpdateStatus()
{
	if (OutputMessage.Len() > 0)
	{
		CertificateList->Reset();
		ProvisionList->Reset();

		// Now split up the log into multiple lines
		TArray<FString> LogLines;
		OutputMessage.ParseIntoArray(LogLines, TEXT("\n"), true);
		
		// format of the line being read here!!
		bool bCerts = false;
		bManuallySelected = false;
		for (int Index = 0; Index < LogLines.Num(); Index++)
		{
			FString& Line = LogLines[Index];
			TArray<FString> Fields;
			Line.ParseIntoArray(Fields, TEXT(","), true);
			if (Line.Contains(TEXT("CERTIFICATE-"), ESearchCase::CaseSensitive))
			{
				CertificatePtr Cert = MakeShareable<FCertificate>(new FCertificate());
				for (int FieldIndex = 0; FieldIndex < Fields.Num(); ++FieldIndex)
				{
					FString Key, Value;
					Fields[FieldIndex].Split(TEXT(":"), &Key, &Value);
					if (Key.Contains("Name"))
					{
						Cert->Name = Value;
					}
					else if (Key.Contains(TEXT("Validity")))
					{
						Cert->Status = Value;
					}
					else if (Key.Contains(TEXT("EndDate")))
					{
						FString Date, Time;
						Value.Split(TEXT("T"), &Date, &Time);
						Cert->Expires = Date;
					}
				}
				CertificatePtr PrevCert = NULL;
				for (int CIndex = 0; CIndex < CertificateList->Num() && !PrevCert.IsValid(); ++CIndex)
				{
					if ((*CertificateList)[CIndex]->Name == Cert->Name)
					{
						PrevCert = (*CertificateList)[CIndex];
						break;
					}
				}

				// check to see if this the one selected in the ini file
				FString OutString;
				SignCertificateProperty->GetValueAsFormattedString(OutString);
				Cert->bManuallySelected = (OutString == Cert->Name);
				bManuallySelected |= Cert->bManuallySelected;
				if (!PrevCert.IsValid())
				{
					CertificateList->Add(Cert);
				}
				else
				{
					FDateTime time1, time2;
					FDateTime::ParseIso8601(*(PrevCert->Expires), time1);
					FDateTime::ParseIso8601(*(Cert->Expires), time2);
					if (time2 > time1)
					{
						PrevCert->Expires = Cert->Expires;
						PrevCert->Status = Cert->Status;
					}
					Cert = NULL;
				}
			}
			else if (Line.Contains(TEXT("PROVISION-"), ESearchCase::CaseSensitive))
			{
				ProvisionPtr Prov = MakeShareable<FProvision>(new FProvision());
				for (int FieldIndex = 0; FieldIndex < Fields.Num(); ++FieldIndex)
				{
					FString Key, Value;
					Fields[FieldIndex].Split(TEXT(":"), &Key, &Value);
					if (Key.Contains("File"))
					{
						Prov->FileName = Value;
					}
					else if (Key.Contains("Name"))
					{
						Prov->Name = Value;
					}
					else if (Key.Contains(TEXT("Validity")))
					{
						Prov->Status = Value;
					}
					else if (Key.Contains(TEXT("Type")))
					{
						Prov->bDistribution = Value.Contains(TEXT("DISTRIBUTION"));
					}
				}

				// check to see if this the one selected in the ini file
				FString OutString;
				MobileProvisionProperty->GetValueAsFormattedString(OutString);
				Prov->bManuallySelected = (OutString == Prov->FileName);
				bManuallySelected |= Prov->bManuallySelected;
				ProvisionList->Add(Prov);
			}
			else if (Line.Contains(TEXT("MATCHED-"), ESearchCase::CaseSensitive))
			{
				for (int FieldIndex = 0; FieldIndex < Fields.Num(); ++FieldIndex)
				{
					FString Key, Value;
					Fields[FieldIndex].Split(TEXT(":"), &Key, &Value);
					if (Key.Contains("File"))
					{
						SelectedFile = Value;
					}
					else if (Key.Contains("Provision"))
					{
						SelectedProvision = Value;
					}
					else if (Key.Contains(TEXT("Cert")))
					{
						SelectedCert = Value;
					}
				}
			}
		}

		FilterLists();
	}
}

void FIOSTargetSettingsCustomization::UpdateSSHStatus()
{
	// updated SSH key
	const UIOSRuntimeSettings* Settings = GetDefault<UIOSRuntimeSettings>();
	const_cast<UIOSRuntimeSettings*>(Settings)->PostInitProperties();
}

void FIOSTargetSettingsCustomization::BuildPListSection(IDetailLayoutBuilder& DetailLayout)
{
	// Info.plist category
	IDetailCategoryBuilder& ProvisionCategory = DetailLayout.EditCategory(TEXT("Mobile Provision"));
	IDetailCategoryBuilder& AppManifestCategory = DetailLayout.EditCategory(TEXT("Info.plist"));
	IDetailCategoryBuilder& BundleCategory = DetailLayout.EditCategory(TEXT("BundleInformation"));
	IDetailCategoryBuilder& OrientationCategory = DetailLayout.EditCategory(TEXT("Orientation"));
	IDetailCategoryBuilder& FileSystemCategory = DetailLayout.EditCategory(TEXT("FileSystem"));
	IDetailCategoryBuilder& RenderCategory = DetailLayout.EditCategory(TEXT("Rendering"));
	IDetailCategoryBuilder& BuildCategory = DetailLayout.EditCategory(TEXT("Build"));
	IDetailCategoryBuilder& OnlineCategory = DetailLayout.EditCategory(TEXT("Online"));
	MobileProvisionProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, MobileProvision));
	BuildCategory.AddProperty(MobileProvisionProperty)
		.Visibility(EVisibility::Hidden);
	SignCertificateProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, SigningCertificate));
	BuildCategory.AddProperty(SignCertificateProperty)
		.Visibility(EVisibility::Hidden);
	AutomaticSigningProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, bAutomaticSigning));

	BuildCategory.AddProperty(AutomaticSigningProperty)
#if PLATFORM_MAC
		.Visibility(EVisibility::Visible);
#else
		.Visibility(EVisibility::Hidden);
#endif // PLATFORM_MAC
	// Remote Server Name Property
	TSharedRef<IPropertyHandle> IOSTeamIDHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, IOSTeamID));
	BuildCategory.AddProperty(IOSTeamIDHandle)
		.Visibility(EVisibility::Hidden);
		BuildCategory.AddCustomRow(LOCTEXT("IOSTeamID", "IOSTeamID"), false)
            .EditCondition(!UXcodeProjectSettings::ShouldDisableIOSSettings(), nullptr)
#if !PLATFORM_MAC
			.Visibility(EVisibility::Hidden)
#endif // PLATFORM_MAC
			.NameContent()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FMargin(0, 1, 0, 1))
					.FillWidth(1.0f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("IOSTeamIDLabel", "IOS Team ID"))
							.Font(DetailLayout.GetDetailFont())
					]
			]
			.ValueContent()
			.MinDesiredWidth(150.0f)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Fill)
					[
						SAssignNew(IOSTeamIDTextBox, SEditableTextBox)
						.IsEnabled(this, &FIOSTargetSettingsCustomization::IsAutomaticSigningEnabled)
						.Text(this, &FIOSTargetSettingsCustomization::GetIOSTeamIDText, IOSTeamIDHandle)
						.Font(DetailLayout.GetDetailFont())
						.SelectAllTextOnCommit(true)
						.SelectAllTextWhenFocused(true)
						.ClearKeyboardFocusOnCommit(false)
						.ToolTipText(IOSTeamIDHandle->GetToolTipText())
						.OnTextCommitted(this, &FIOSTargetSettingsCustomization::OnIOSTeamIDTextChanged, IOSTeamIDHandle)
					]
			];

/*	ProvisionCategory.AddCustomRow(TEXT("Certificate Request"), false)
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, 1, 0, 1))
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RequestLabel", "Certificate Request"))
				.Font(DetailLayout.GetDetailFont())
			]
		]
	.ValueContent()
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.OnClicked(this, &FIOSTargetSettingsCustomization::OnCertificateRequestClicked)
				[
					SNew(STextBlock)
					.Text(FText::FromString("Create Certificate Request and a Key Pair"))
				]
			]
		];*/

    ProvisionCategory.AddCustomRow(LOCTEXT("ModernXcodeSigningLabel", "XcodeSigning"), false)
        .Visibility(UXcodeProjectSettings::ShouldDisableIOSSettings() ? EVisibility::Visible : EVisibility::Hidden)
        .WholeRowWidget
        .MinDesiredWidth(0.f)
        .MaxDesiredWidth(0.f)
        .HAlign(HAlign_Fill)
        [
            SNew( STextBlock )
            .Text( LOCTEXT( "ModernXcodeSigningText", "Code signing is handled by \"Xcode Projects\" section") )
            .AutoWrapText( true )
        ];
    
	ProvisionCategory.AddCustomRow(LOCTEXT("ProvisionLabel", "Provision"), false)
        .EditCondition(!UXcodeProjectSettings::ShouldDisableIOSSettings(), nullptr)
		.WholeRowWidget
		.MinDesiredWidth(0.f)
		.MaxDesiredWidth(0.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(ProvisionInfoSwitcher, SWidgetSwitcher)
				.WidgetIndex(0)
				// searching for provisions
				+SWidgetSwitcher::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.Padding(4)
					[
						SNew( STextBlock )
						.Text( LOCTEXT( "ProvisionViewerFindingProvisions", "Please wait while we gather information." ) )
						.AutoWrapText( true )
					]
				]
				// importing a provision
				+SWidgetSwitcher::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(SBorder)
						.Padding(4)
						[
							SNew( STextBlock )
							.Text( LOCTEXT( "ProvisionViewerImportingProvisions", "Importing Provision.  Please wait..." ) )
							.AutoWrapText( true )
						]
					]
				// no provisions found or no valid provisions
				+SWidgetSwitcher::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.Padding(4)
					[
						SNew( STextBlock )
						.Text( LOCTEXT( "ProvisionViewerNoValidProvisions", "No Provisions Found. Please Import a Provision." ) )
						.AutoWrapText( true )
					]
				]
				+SWidgetSwitcher::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(FMargin(10, 10, 10, 10))
					.AutoHeight()
					[
						SAssignNew(ProvisionListView, SListView<ProvisionPtr>)
						.ListItemsSource(&FilteredProvisionList)
						.OnGenerateRow(this, &FIOSTargetSettingsCustomization::HandleProvisionListGenerateRow)
						.SelectionMode(ESelectionMode::None)
						.HeaderRow
						(
						SNew(SHeaderRow)
						+ SHeaderRow::Column("Selected")
						.DefaultLabel(FText::GetEmpty())
						.FixedWidth(30.0f)
						+ SHeaderRow::Column("Name")
						.DefaultLabel(LOCTEXT("ProvisionListNameColumnHeader", "Provision"))
						.FillWidth(1.0f)
						+ SHeaderRow::Column("File")
						.DefaultLabel(LOCTEXT("ProvisionListFileColumnHeader", "File"))
						+ SHeaderRow::Column("Status")
						.DefaultLabel(LOCTEXT("ProvisionListStatusColumnHeader", "Status"))
						+ SHeaderRow::Column("Distribution")
						.DefaultLabel(LOCTEXT("ProvisionListDistributionColumnHeader", "Distribution"))
						.FixedWidth(75.0f)
						)
					]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 6.0f, 0.0f, 4.0f)
						[
							SNew(SSeparator)
							.Orientation(Orient_Horizontal)
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SRichTextBlock)
								.Text(LOCTEXT("ProvisionMessage", "<RichTextBlock.TextHighlight>Note</>: If no provision is selected the one in green will be used to provision the IPA."))
								.TextStyle(FAppStyle::Get(), "MessageLog")
								.DecoratorStyleSet(&FAppStyle::Get())
								.AutoWrapText(true)
							]

							+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.HAlign(HAlign_Right)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("ViewLabel", "View:"))
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(8.0f, 0.0f)
								[
									// all provisions hyper link
									SNew(SHyperlink)
									.OnNavigate(this, &FIOSTargetSettingsCustomization::HandleAllProvisionsHyperlinkNavigate, true)
									.Text(LOCTEXT("AllProvisionsHyperLinkLabel", "All"))
									.ToolTipText(LOCTEXT("AllProvisionsButtonTooltip", "View all provisions."))
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									// valid provisions hyper link
									SNew(SHyperlink)
									.OnNavigate(this, &FIOSTargetSettingsCustomization::HandleAllProvisionsHyperlinkNavigate, false)
									.Text(LOCTEXT("ValidProvisionsHyperlinkLabel", "Valid Only"))
									.ToolTipText(LOCTEXT("ValidProvisionsHyperlinkTooltip", "View Valid provisions."))
								]
						]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.Padding(FMargin(0, 5, 0, 10))
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked(this, &FIOSTargetSettingsCustomization::OnInstallProvisionClicked)
						.IsEnabled(this, &FIOSTargetSettingsCustomization::IsImportEnabled)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ImportProvision", "Import Provision"))
						]
					]
			]
		];

	ProvisionCategory.AddCustomRow(LOCTEXT("CertificateLabel", "Certificate"), false)
        .EditCondition(!UXcodeProjectSettings::ShouldDisableIOSSettings(), nullptr)
		.WholeRowWidget
		.MinDesiredWidth(0.f)
		.MaxDesiredWidth(0.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(CertificateInfoSwitcher, SWidgetSwitcher)
				.WidgetIndex(0)
				// searching for provisions
				+SWidgetSwitcher::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.Padding(4)
					[
						SNew( STextBlock )
						.Text( LOCTEXT( "CertificateViewerFindingProvisions", "Please wait while we gather information." ) )
						.AutoWrapText( true )
					]
				]
				// importing certificate
				+SWidgetSwitcher::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(SBorder)
						.Padding(4)
						[
							SNew( STextBlock )
							.Text( LOCTEXT( "CertificateViewerImportingCertificate", "Importing Certificate.  Please wait..." ) )
							.AutoWrapText( true )
						]
					]
				// no provisions found or no valid provisions
				+SWidgetSwitcher::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(SBorder)
						.Padding(4)
						[
							SNew( STextBlock )
							.Text( LOCTEXT( "CertificateViewerNoValidProvisions", "No Certificates Found.  Please Import a Certificate." ) )
							.AutoWrapText( true )
						]
					]
				+SWidgetSwitcher::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.Padding(FMargin(10, 10, 10, 10))
							.FillWidth(1.0f)
							[
								SAssignNew(CertificateListView, SListView<CertificatePtr>)
								.ListItemsSource(&FilteredCertificateList)
								.OnGenerateRow(this, &FIOSTargetSettingsCustomization::HandleCertificateListGenerateRow)
								.SelectionMode(ESelectionMode::None)
								.HeaderRow
								(
									SNew(SHeaderRow)
									+ SHeaderRow::Column("Selected")
									.DefaultLabel(FText::GetEmpty())
									.FixedWidth(30.0f)
									+ SHeaderRow::Column("Name")
									.DefaultLabel(LOCTEXT("CertificateListNameColumnHeader", "Certificate"))
									+ SHeaderRow::Column("Status")
									.DefaultLabel(LOCTEXT("CertificateListStatusColumnHeader", "Status"))
									.FixedWidth(75.0f)
									+ SHeaderRow::Column("Expires")
									.DefaultLabel(LOCTEXT("CertificateListExpiresColumnHeader", "Expires"))
									.FixedWidth(75.0f)
								)
							]
						]
						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 6.0f, 0.0f, 4.0f)
							[
								SNew(SSeparator)
								.Orientation(Orient_Horizontal)
							]
						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SRichTextBlock)
									.Text(LOCTEXT("CertificateMessage", "<RichTextBlock.TextHighlight>Note</>: If no certificate is selected then the one in green will be used to sign the IPA."))
									.TextStyle(FAppStyle::Get(), "MessageLog")
									.DecoratorStyleSet(&FAppStyle::Get())
									.AutoWrapText(true)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.HAlign(HAlign_Right)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("ViewLabel", "View:"))
								]

								+ SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(8.0f, 0.0f)
									[
										// all provisions hyper link
										SNew(SHyperlink)
										.OnNavigate(this, &FIOSTargetSettingsCustomization::HandleAllCertificatesHyperlinkNavigate, true)
										.Text(LOCTEXT("AllCertificatesHyperLinkLabel", "All"))
										.ToolTipText(LOCTEXT("AllCertificatesButtonTooltip", "View all certificates."))
									]

								+ SHorizontalBox::Slot()
									.AutoWidth()
									[
										// valid provisions hyper link
										SNew(SHyperlink)
										.OnNavigate(this, &FIOSTargetSettingsCustomization::HandleAllCertificatesHyperlinkNavigate, false)
										.Text(LOCTEXT("ValidCertificatesHyperlinkLabel", "Valid Only"))
										.ToolTipText(LOCTEXT("ValidCertificatesHyperlinkTooltip", "View Valid certificates."))
									]
							]
					]
				]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0, 5, 0, 10))
				.AutoWidth()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked(this, &FIOSTargetSettingsCustomization::OnInstallCertificateClicked)
					.IsEnabled(this, &FIOSTargetSettingsCustomization::IsImportEnabled)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ImportCertificate", "Import Certificate"))
					]
				]
			]
		];

	// Show properties that are gated by the plist being present and writable
	RunningIPPProcess = false;

#define SETUP_SOURCEONLY_PROP(PropName, Category) \
	{ \
		TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, PropName)); \
		Category.AddProperty(PropertyHandle) \
			.IsEnabled(FEngineBuildSettings::IsSourceDistribution()) \
			.ToolTip(FEngineBuildSettings::IsSourceDistribution() ? PropertyHandle->GetToolTipText() : FIOSTargetSettingsCustomizationConstants::DisabledTip); \
	}

#define SETUP_PLIST_PROP(PropName, Category) \
	{ \
		TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, PropName)); \
		Category.AddProperty(PropertyHandle); \
	}

#define SETUP_STATUS_PROP(PropName, Category) \
	{ \
		TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, PropName)); \
		Category.AddProperty(PropertyHandle) \
		.Visibility(EVisibility::Hidden); \
		Category.AddCustomRow(LOCTEXT("BundleIdentifier", "BundleIdentifier"), false) \
        .EditCondition(!UXcodeProjectSettings::ShouldDisableIOSSettings(), nullptr) \
		.NameContent() \
		[ \
			SNew(SHorizontalBox) \
			+ SHorizontalBox::Slot() \
			.Padding(FMargin(0, 1, 0, 1)) \
			.FillWidth(1.0f) \
			[ \
				SNew(STextBlock) \
				.Text(LOCTEXT("BundleIdentifierLabel", "Bundle Identifier")) \
				.Font(DetailLayout.GetDetailFont()) \
			]\
		] \
		.ValueContent() \
		.MinDesiredWidth( 0.0f ) \
		.MaxDesiredWidth( 0.0f ) \
		[ \
			SNew(SHorizontalBox) \
			+ SHorizontalBox::Slot() \
			.FillWidth(1.0f) \
			.HAlign(HAlign_Fill) \
			[ \
				SAssignNew(BundleIdTextBox, SEditableTextBox) \
				.IsEnabled(this, &FIOSTargetSettingsCustomization::IsImportEnabled) \
				.Text(this, &FIOSTargetSettingsCustomization::GetBundleText, PropertyHandle) \
				.Font(DetailLayout.GetDetailFont()) \
				.SelectAllTextOnCommit( true ) \
				.SelectAllTextWhenFocused( true ) \
				.ClearKeyboardFocusOnCommit(false) \
				.ToolTipText(PropertyHandle->GetToolTipText()) \
				.OnTextCommitted(this, &FIOSTargetSettingsCustomization::OnBundleIdentifierChanged, PropertyHandle) \
				.OnTextChanged(this, &FIOSTargetSettingsCustomization::OnBundleIdentifierTextChanged, ETextCommit::Default, PropertyHandle) \
			] \
		]; \
	}

	const UIOSRuntimeSettings& Settings = *GetDefault<UIOSRuntimeSettings>();

	FSimpleDelegate OnUpdateShaderStandardWarning = FSimpleDelegate::CreateSP(this, &FIOSTargetSettingsCustomization::UpdateShaderStandardWarning);
	FSimpleDelegate OnUpdateOSVersionWarning = FSimpleDelegate::CreateSP(this, &FIOSTargetSettingsCustomization::UpdateOSVersionWarning);
	FSimpleDelegate OnEnableMetalMobileSM5 = FSimpleDelegate::CreateSP(this, &FIOSTargetSettingsCustomization::UpdateMetalMobileWarning);
	FSimpleDelegate OnEnableMetalMobileSM6 = FSimpleDelegate::CreateSP(this, &FIOSTargetSettingsCustomization::UpdateMetalMobileWarning);

	SETUP_PLIST_PROP(BundleDisplayName, BundleCategory);
	SETUP_PLIST_PROP(BundleName, BundleCategory);
	SETUP_STATUS_PROP(BundleIdentifier, BundleCategory);
	SETUP_PLIST_PROP(VersionInfo, BundleCategory);
	SETUP_PLIST_PROP(bSupportsPortraitOrientation, OrientationCategory);
	SETUP_PLIST_PROP(bSupportsUpsideDownOrientation, OrientationCategory);
	SETUP_PLIST_PROP(bSupportsLandscapeLeftOrientation, OrientationCategory);
	SETUP_PLIST_PROP(bSupportsLandscapeRightOrientation, OrientationCategory);
	SETUP_PLIST_PROP(PreferredLandscapeOrientation, OrientationCategory);
	
	SETUP_PLIST_PROP(bSupportsITunesFileSharing, FileSystemCategory);
	SETUP_PLIST_PROP(bSupportsFilesApp, FileSystemCategory);
	
	SETUP_PLIST_PROP(bSupportsMetal, RenderCategory);
	
	MRTPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, bSupportsMetalMobileSM5));
	MRTPropertyHandle->SetOnPropertyValueChanged(OnEnableMetalMobileSM5);
	RenderCategory.AddProperty(MRTPropertyHandle);

	SM6PropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, bSupportsMetalMobileSM6));
	SM6PropertyHandle->SetOnPropertyValueChanged(OnEnableMetalMobileSM6);
	RenderCategory.AddProperty(SM6PropertyHandle);

	SETUP_SOURCEONLY_PROP(bEnableRemoteNotificationsSupport, OnlineCategory)
//	SETUP_SOURCEONLY_PROP(bAutomaticSigning, ProvisionCategory)

    // Handle max. shader version a little specially.
    {
        ShaderVersionPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, MetalLanguageVersion));
		ShaderVersionPropertyHandle->SetOnPropertyValueChanged(OnUpdateShaderStandardWarning);
		
		// Drop-downs for setting type of lower and upper bound normalization
		IDetailPropertyRow& ShaderVersionPropertyRow = RenderCategory.AddProperty(ShaderVersionPropertyHandle.ToSharedRef());
		ShaderVersionPropertyRow.CustomWidget()
		.NameContent()
		[
			ShaderVersionPropertyHandle->CreatePropertyNameWidget()
		 ]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FIOSTargetSettingsCustomization::OnGetShaderVersionContent)
				.ContentPadding(FMargin( 2.0f, 2.0f ))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FIOSTargetSettingsCustomization::GetShaderVersionDesc)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			 ]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(2)
			[
				SAssignNew(ShaderVersionWarningTextBox, SErrorText)
				.AutoWrapText(true)
			]
		];
		
		UpdateShaderStandardWarning();
    }

	// Handle Min IOS version a little specially.
	{
		MinOSPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, MinimumiOSVersion));
		MinOSPropertyHandle->SetOnPropertyValueChanged(OnUpdateOSVersionWarning);

		// Drop-downs for setting type of lower and upper bound normalization
		IDetailPropertyRow& MinOSPropertyRow = BuildCategory.AddProperty(MinOSPropertyHandle.ToSharedRef());
		MinOSPropertyRow.CustomWidget()
		.NameContent()
		[
			MinOSPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FIOSTargetSettingsCustomization::OnGetMinVersionContent)
				.ContentPadding(FMargin(2.0f, 2.0f))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FIOSTargetSettingsCustomization::GetMinVersionDesc)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(2)
			[
				SAssignNew(IOSVersionWarningTextBox, SErrorText)
				.AutoWrapText(true)
			]
		];

		UpdateOSVersionWarning();
	}
#undef SETUP_SOURCEONLY_PROP
}


void FIOSTargetSettingsCustomization::BuildRemoteBuildingSection(IDetailLayoutBuilder& DetailLayout)
{
#if PLATFORM_WINDOWS
	IDetailCategoryBuilder& BuildCategory = DetailLayout.EditCategory(TEXT("Remote Build"));

	// Sub group we wish to add remote building options to.
	FText RemoteBuildingGroupName = LOCTEXT("RemoteBuildingGroupName", "Remote Build Options");
	IDetailGroup& RemoteBuildingGroup = BuildCategory.AddGroup(*RemoteBuildingGroupName.ToString(), RemoteBuildingGroupName, false);


	// Remote Server Name Property
	TSharedRef<IPropertyHandle> RemoteServerNamePropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, RemoteServerName));
	IDetailPropertyRow& RemoteServerNamePropertyRow = RemoteBuildingGroup.AddPropertyRow(RemoteServerNamePropertyHandle);
	RemoteServerNamePropertyRow
		.ToolTip(RemoteServerNamePropertyHandle->GetToolTipText())
		.CustomWidget()
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0, 1, 0, 1))
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("RemoteServerNameLabel", "Remote Server Name"))
		.Font(DetailLayout.GetDetailFont())
		]
		]
	.ValueContent()
		.MinDesiredWidth(150.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 8.0f))
		[
			SNew(SEditableTextBox)
			.IsEnabled(this, &FIOSTargetSettingsCustomization::IsImportEnabled)
		.Text(this, &FIOSTargetSettingsCustomization::GetBundleText, RemoteServerNamePropertyHandle)
		.Font(DetailLayout.GetDetailFont())
		.SelectAllTextOnCommit(true)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false)
		.ToolTipText(RemoteServerNamePropertyHandle->GetToolTipText())
		.OnTextCommitted(this, &FIOSTargetSettingsCustomization::OnRemoteServerChanged, RemoteServerNamePropertyHandle)
		]

		];



	// Add Use RSync Property
	TSharedRef<IPropertyHandle> UseRSyncPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, bUseRSync));
	BuildCategory.AddProperty(UseRSyncPropertyHandle)
		.Visibility(EVisibility::Hidden);

	// Add RSync Username Property
	TSharedRef<IPropertyHandle> RSyncUsernamePropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, RSyncUsername));
	IDetailPropertyRow& RSyncUsernamePropertyRow = RemoteBuildingGroup.AddPropertyRow(RSyncUsernamePropertyHandle);
	RSyncUsernamePropertyRow
		.ToolTip(RSyncUsernamePropertyHandle->GetToolTipText())
		.CustomWidget()
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0, 1, 0, 1))
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("RSyncUserNameLabel", "RSync User Name"))
		.Font(DetailLayout.GetDetailFont())
		]
		]
	.ValueContent()
		.MinDesiredWidth(150.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 8.0f))
		[
			SNew(SEditableTextBox)
			.IsEnabled(this, &FIOSTargetSettingsCustomization::IsImportEnabled)
		.Text(this, &FIOSTargetSettingsCustomization::GetBundleText, RSyncUsernamePropertyHandle)
		.Font(DetailLayout.GetDetailFont())
		.SelectAllTextOnCommit(true)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false)
		.ToolTipText(RSyncUsernamePropertyHandle->GetToolTipText())
		.OnTextCommitted(this, &FIOSTargetSettingsCustomization::OnRemoteServerChanged, RSyncUsernamePropertyHandle)
		]
		];


	// Add Remote Server Override Build Path 
	TSharedRef<IPropertyHandle> RemoteServerOverrideBuildPathPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, RemoteServerOverrideBuildPath));
	IDetailPropertyRow& RemoteServerOverrideBuildPathPropertyRow = RemoteBuildingGroup.AddPropertyRow(RemoteServerOverrideBuildPathPropertyHandle);
	RemoteServerOverrideBuildPathPropertyRow
		.ToolTip(RemoteServerOverrideBuildPathPropertyHandle->GetToolTipText());

	// Add existing SSH path label.
	TSharedRef<IPropertyHandle> SSHPrivateKeyLocationPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, SSHPrivateKeyLocation));
	IDetailPropertyRow& SSHPrivateKeyLocationPropertyRow = RemoteBuildingGroup.AddPropertyRow(SSHPrivateKeyLocationPropertyHandle);
	SSHPrivateKeyLocationPropertyRow
		.ToolTip(SSHPrivateKeyLocationPropertyHandle->GetToolTipText());

	// cwRsync path
	TSharedRef<IPropertyHandle> CwRsyncOverridePathPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, CwRsyncInstallPath));
	IDetailPropertyRow& CwRsyncOverridePathPropertyRow = RemoteBuildingGroup.AddPropertyRow(CwRsyncOverridePathPropertyHandle);

	const FText GenerateSSHText = LOCTEXT("GenerateSSHKey", "Generate SSH Key");

	// Add a generate key button
	RemoteBuildingGroup.AddWidgetRow()
		.FilterString(GenerateSSHText)
		.WholeRowWidget
		.MinDesiredWidth(0.f)
		.MaxDesiredWidth(0.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0, 5, 0, 10))
		.AutoWidth()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.OnClicked(this, &FIOSTargetSettingsCustomization::OnGenerateSSHKey, true)
		.IsEnabled(this, &FIOSTargetSettingsCustomization::IsImportEnabled)
		[
			SNew(STextBlock)
			.Text(GenerateSSHText)
		]
		]
		]
		];
#endif
}

void FIOSTargetSettingsCustomization::BuildSecondaryRemoteMacBuildingSection(IDetailLayoutBuilder& DetailLayout)
{
#if PLATFORM_WINDOWS
	IDetailCategoryBuilder& BuildCategory = DetailLayout.EditCategory(TEXT("Remote Build"));

	// Sub group we wish to add remote building options to.
	FText RemoteBuildingGroupName = LOCTEXT("SecondaryRemoteBuildingGroupName", "Secondary Remote Build Options");
	IDetailGroup& RemoteBuildingGroup = BuildCategory.AddGroup(*RemoteBuildingGroupName.ToString(), RemoteBuildingGroupName, false);

	// Remote Server Name Property
	TSharedRef<IPropertyHandle> RemoteServerNamePropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, SecondaryRemoteServerName));

	IDetailPropertyRow& RemoteServerNamePropertyRow = RemoteBuildingGroup.AddPropertyRow(RemoteServerNamePropertyHandle);
	RemoteServerNamePropertyRow
		.ToolTip(RemoteServerNamePropertyHandle->GetToolTipText())
		.CustomWidget()
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0, 1, 0, 1))
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SecondaryRemoteServerNameLabel", "Secondary Remote Server Name"))
		.Font(DetailLayout.GetDetailFont())
		]
		]
	.ValueContent()
		.MinDesiredWidth(150.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 8.0f))
		[
			SNew(SEditableTextBox)
			.IsEnabled(this, &FIOSTargetSettingsCustomization::IsImportEnabled)
		.Text(this, &FIOSTargetSettingsCustomization::GetBundleText, RemoteServerNamePropertyHandle)
		.Font(DetailLayout.GetDetailFont())
		.SelectAllTextOnCommit(true)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false)
		.ToolTipText(RemoteServerNamePropertyHandle->GetToolTipText())
		.OnTextCommitted(this, &FIOSTargetSettingsCustomization::OnRemoteServerChanged, RemoteServerNamePropertyHandle)
		]

		];



	// Add Use RSync Property
	TSharedRef<IPropertyHandle> UseRSyncPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, bUseRSync));
	BuildCategory.AddProperty(UseRSyncPropertyHandle)
		.Visibility(EVisibility::Hidden);

	// Add RSync Username Property
	TSharedRef<IPropertyHandle> RSyncUsernamePropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, SecondaryRSyncUsername));
	IDetailPropertyRow& RSyncUsernamePropertyRow = RemoteBuildingGroup.AddPropertyRow(RSyncUsernamePropertyHandle);
	RSyncUsernamePropertyRow
		.ToolTip(RSyncUsernamePropertyHandle->GetToolTipText())
		.CustomWidget()
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0, 1, 0, 1))
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SecondaryRSyncUserNameLabel", "Secondary RSync User Name"))
		.Font(DetailLayout.GetDetailFont())
		]
		]
	.ValueContent()
		.MinDesiredWidth(150.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 8.0f))
		[
			SNew(SEditableTextBox)
			.IsEnabled(this, &FIOSTargetSettingsCustomization::IsImportEnabled)
		.Text(this, &FIOSTargetSettingsCustomization::GetBundleText, RSyncUsernamePropertyHandle)
		.Font(DetailLayout.GetDetailFont())
		.SelectAllTextOnCommit(true)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false)
		.ToolTipText(RSyncUsernamePropertyHandle->GetToolTipText())
		.OnTextCommitted(this, &FIOSTargetSettingsCustomization::OnRemoteServerChanged, RSyncUsernamePropertyHandle)
		]
		];


	// Add Remote Server Override Build Path 
	TSharedRef<IPropertyHandle> RemoteServerOverrideBuildPathPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, SecondaryRemoteServerOverrideBuildPath));
	IDetailPropertyRow& RemoteServerOverrideBuildPathPropertyRow = RemoteBuildingGroup.AddPropertyRow(RemoteServerOverrideBuildPathPropertyHandle);
	RemoteServerOverrideBuildPathPropertyRow
		.ToolTip(RemoteServerOverrideBuildPathPropertyHandle->GetToolTipText());

	// Add existing SSH path label.
	TSharedRef<IPropertyHandle> SSHPrivateKeyLocationPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, SecondarySSHPrivateKeyLocation));
	IDetailPropertyRow& SSHPrivateKeyLocationPropertyRow = RemoteBuildingGroup.AddPropertyRow(SSHPrivateKeyLocationPropertyHandle);
	SSHPrivateKeyLocationPropertyRow
		.ToolTip(SSHPrivateKeyLocationPropertyHandle->GetToolTipText());

	// cwRsync path
	TSharedRef<IPropertyHandle> CwRsyncOverridePathPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, SecondaryCwRsyncInstallPath));
	IDetailPropertyRow& CwRsyncOverridePathPropertyRow = RemoteBuildingGroup.AddPropertyRow(CwRsyncOverridePathPropertyHandle);

	const FText GenerateSSHText = LOCTEXT("SecondaryGenerateSSHKey", "Secondary Generate SSH Key");

	// Add a generate key button
	RemoteBuildingGroup.AddWidgetRow()
		.FilterString(GenerateSSHText)
		.WholeRowWidget
		.MinDesiredWidth(0.f)
		.MaxDesiredWidth(0.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(FMargin(0, 5, 0, 10))
		.AutoWidth()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.OnClicked(this, &FIOSTargetSettingsCustomization::OnGenerateSSHKey, false)
		.IsEnabled(this, &FIOSTargetSettingsCustomization::IsImportEnabled)
		[
			SNew(STextBlock)
			.Text(GenerateSSHText)
		]
		]
		]
		];
#endif
}

void FIOSTargetSettingsCustomization::BuildIconSection(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& RequiredIconCategoryIOS = DetailLayout.EditCategory("RequiredIOSIcons", LOCTEXT("RequiredIOSIcons", "Required Icons (iPhone and iPad)"));
    IDetailCategoryBuilder& OptionalIconCategoryIOS = DetailLayout.EditCategory("OptionalIOSIcons", LOCTEXT("OptionalIOSIcons", "Optional Icons (iPhone and iPad)"));

	// Add the iOS icons
	for (const FPlatformIconInfo& Info : IconNames)
	{
		FVector2D IconImageMaxSize(Info.IconRequiredSize);
		IconImageMaxSize.X = FMath::Min(IconImageMaxSize.X, 150.0f);
		IconImageMaxSize.Y = FMath::Min(IconImageMaxSize.Y, 150.0f);
		IDetailCategoryBuilder& IconCategory = (Info.RequiredState == FPlatformIconInfo::Required) ? RequiredIconCategoryIOS : OptionalIconCategoryIOS;
		BuildImageRow(DetailLayout, IconCategory, Info, IconImageMaxSize);
	}

	// Add the tvOS content
    IDetailCategoryBuilder& RequiredIconCategoryTvOS = DetailLayout.EditCategory("RequiredTVOSAssets", LOCTEXT("RequiredTVOSAssets", "Required Assets (AppleTV)"));
    IDetailCategoryBuilder& OptionalIconCategoryTvOS = DetailLayout.EditCategory("OptionalTVOSAssets", LOCTEXT("OptionalTVOSAssets", "Optional Assets (AppleTV)"));

	const FVector2D TvOSImageMaxSize(150.0f, 150.0f);
	for (const FPlatformIconInfo& Info : TvOSImageNames)
	{
        IDetailCategoryBuilder& ImageCategory = (Info.RequiredState == FPlatformIconInfo::Required) ? RequiredIconCategoryTvOS : OptionalIconCategoryTvOS;
		BuildImageRow(DetailLayout, ImageCategory, Info, TvOSImageMaxSize, true);
	}
    
    
    // Add the launch images
    IDetailCategoryBuilder& LaunchImageCategory = DetailLayout.EditCategory(FName("LaunchScreen"));
    const FVector2D LaunchImageMaxSize(150.0f, 150.0f);
    for (const FPlatformIconInfo& Info : LaunchImageNames)
    {
        BuildImageRow(DetailLayout, LaunchImageCategory, Info, LaunchImageMaxSize);
    }

}


FReply FIOSTargetSettingsCustomization::OpenPlistFolder()
{
	const FString EditPlistFolder = FPaths::ConvertRelativePathToFull(FPaths::GetPath(GameInfoPath));
	FPlatformProcess::ExploreFolder(*EditPlistFolder);

	return FReply::Handled();
}

void FIOSTargetSettingsCustomization::CopySetupFilesIntoProject()
{
	// First copy the plist, it must get copied
	FText ErrorMessage;
	if (!SourceControlHelpers::CopyFileUnderSourceControl(GameInfoPath, EngineInfoPath, LOCTEXT("InfoPlist", "Info.plist"), /*out*/ ErrorMessage))
	{
		FNotificationInfo Info(ErrorMessage);
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
	else
	{
		// Now try to copy all of the icons, etc... (these can be ignored if the file already exists)
		TArray<FPlatformIconInfo> Graphics;
		Graphics.Empty(IconNames.Num() + LaunchImageNames.Num());
		Graphics.Append(IconNames);
		Graphics.Append(LaunchImageNames);

		for (const FPlatformIconInfo& Info : Graphics)
		{
			const FString EngineImagePath = EngineGraphicsPath / Info.IconPath;
			const FString ProjectImagePath = GameGraphicsPath / Info.IconPath;

			if (!FPaths::FileExists(ProjectImagePath))
			{
				SourceControlHelpers::CopyFileUnderSourceControl(ProjectImagePath, EngineImagePath, Info.IconName, /*out*/ ErrorMessage);
			}
		}
	}

	SavedLayoutBuilder->ForceRefreshDetails();
}

void FIOSTargetSettingsCustomization::BuildImageRow(IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& Category, const FPlatformIconInfo& Info, const FVector2D& MaxDisplaySize, bool bIsTVOS)
{
    FString AutomaticImagePath = EngineGraphicsPath / Info.IconPath;
	FString TargetImagePath = GameGraphicsPath / Info.IconPath;
    FString SourceImagePath = FPaths::GetPath(FPaths::GetProjectFilePath()) + TEXT("/Build/IOS/Resources/Graphics/Icon1024.png");

	if (bIsTVOS)
	{
		AutomaticImagePath = TVOSEngineGraphicsPath / Info.IconPath;
		TargetImagePath = TVOSGameGraphicsPath / Info.IconPath;
        SourceImagePath = FPaths::GetPath(FPaths::GetProjectFilePath()) + TEXT("/Build/TVOS/Resources/Graphics/Icon_Large_Front.png");

        if (Info.IconName.ToString().Contains("Top Shelf"))
        {
             SourceImagePath = FPaths::GetPath(FPaths::GetProjectFilePath()) + TEXT("/Build/TVOS/Resources/Graphics/TopShelfWide-2320x720@2x.png");
        }
	}

    if (Info.RequiredState == FPlatformIconInfo::Required)
    {
        Category.AddCustomRow(Info.IconName)
            .EditCondition(!UXcodeProjectSettings::ShouldDisableIOSSettings(), nullptr)
            .NameContent()
            [
                SNew(SHorizontalBox)
                + SHorizontalBox::Slot()
                .Padding(FMargin(0, 1, 0, 1))
                .FillWidth(1.0f)
                [
                    SNew(STextBlock)
                    .Text(Info.IconName)
                    .Font(DetailLayout.GetDetailFont())
                    // IconDescription is not used, repurpose for tooltip
                    .ToolTipText(Info.IconDescription)
                ]
            ]
            .ValueContent()
            .MaxDesiredWidth(400.0f)
            .MinDesiredWidth(100.0f)
            [
                SNew(SHorizontalBox)
                + SHorizontalBox::Slot()
                .FillWidth(1.0f)
                .VAlign(VAlign_Center)
                [
                    SNew(SExternalImageReference, AutomaticImagePath, TargetImagePath)
                    .RequiredSize(Info.IconRequiredSize)
                    .MaxDisplaySize(MaxDisplaySize)
					.GenerateImageVisibility(this, &FIOSTargetSettingsCustomization::ShouldShowGenerateButtonForIcon, Info.IconName.ToString().Contains("Launch Screen Image") || Info.IconName.ToString().Contains("Marketing Icon") ||
                                             Info.IconName.ToString().Contains("Icon Large Front (1280x768)") || Info.IconName.ToString().Contains("2x Top Shelf Wide (4640x1440)"), SourceImagePath)
					.GenerateImageToolTipText(LOCTEXT("GenerateFromOtherIcon", "Generate from Bigger Image (see image tooltip)"))
					.OnGenerateImageClicked(this, &FIOSTargetSettingsCustomization::OnGenerateImageClicked, SourceImagePath, TargetImagePath, Info.IconRequiredSize)
                    .DeleteTargetWhenDefaultChosen(true)
                ]
            ];
    }
    else
    {
        Category.AddCustomRow(Info.IconName)
            .EditCondition(!UXcodeProjectSettings::ShouldDisableIOSSettings(), nullptr)
            .NameContent()
            [
                SNew(SHorizontalBox)
                + SHorizontalBox::Slot()
                .Padding(FMargin(0, 1, 0, 1))
                .FillWidth(1.0f)
                [
                    SNew(STextBlock)
                    .Text(Info.IconName)
                    .Font(DetailLayout.GetDetailFont())
                    // IconDescription is not used, repurpose for tooltip
                    .ToolTipText(Info.IconDescription)
                ]
            ]
            .ValueContent()
            .MaxDesiredWidth(400.0f)
            .MinDesiredWidth(100.0f)
            [
                SNew(SHorizontalBox)
                + SHorizontalBox::Slot()
                .FillWidth(1.0f)
                .VAlign(VAlign_Center)
                [
                    SNew(SExternalImageReference, "", TargetImagePath)
                    .RequiredSize(Info.IconRequiredSize)
                    .MaxDisplaySize(MaxDisplaySize)
					.GenerateImageVisibility(this, &FIOSTargetSettingsCustomization::ShouldShowGenerateButtonForIcon, Info.IconName.ToString().Contains("Launch Screen Image") || Info.IconName.ToString().Contains("Marketing Icon") ||
                                             Info.IconName.ToString().Contains("Icon Large Front (1280x768)") || Info.IconName.ToString().Contains("2x Top Shelf Wide (4640x1440)"), SourceImagePath)
					.GenerateImageToolTipText(LOCTEXT("GenerateFromOtherIcon", "Generate from Bigger Image (see image tooltip)"))
					.OnGenerateImageClicked(this, &FIOSTargetSettingsCustomization::OnGenerateImageClicked, SourceImagePath, TargetImagePath, Info.IconRequiredSize)
				]
            ];
    }
}

void FIOSTargetSettingsCustomization::FindRequiredFiles()
{
	const UIOSRuntimeSettings& Settings = *GetDefault<UIOSRuntimeSettings>();
	FString BundleIdentifier = Settings.BundleIdentifier.Replace(*gProjectNameText, FApp::GetProjectName());
	BundleIdentifier = BundleIdentifier.Replace(TEXT("_"), TEXT(""));
#if PLATFORM_MAC
	FString CmdExe = TEXT("/bin/sh");
	FString ScriptPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Build/BatchFiles/RunDotnet.sh"));
	FString IPPPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/DotNET/IOS/IPhonePackagerCLI.dll"));
	FString CommandLine = FString::Printf(TEXT("\"%s\" \"%s\" certificates Engine -bundlename \"%s\""), *ScriptPath, *IPPPath, *(BundleIdentifier));
#else
	FString CmdExe = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/DotNET/IOS/IPhonePackager.exe"));
	FString CommandLine = FString::Printf(TEXT("certificates Engine -bundlename \"%s\""), *(BundleIdentifier));
#endif
	IPPProcess = MakeShareable(new FMonitoredProcess(CmdExe, CommandLine, true));
	OutputMessage = TEXT("");
	IPPProcess->OnOutput().BindStatic(&OnOutput);
	IPPProcess->Launch();
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FIOSTargetSettingsCustomization::UpdateStatusDelegate), 1.0f);
	if (ProvisionInfoSwitcher.IsValid())
	{
		ProvisionInfoSwitcher->SetActiveWidgetIndex(0);
	}
	if (CertificateInfoSwitcher.IsValid())
	{
		CertificateInfoSwitcher->SetActiveWidgetIndex(0);
	}
	RunningIPPProcess = true;
}

FReply FIOSTargetSettingsCustomization::OnInstallProvisionClicked()
{
	// pass the file to IPP to install
	FString ProjectPath = FPaths::IsProjectFilePathSet() ? FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath()) : FPaths::RootDir() / FApp::GetProjectName() / FApp::GetProjectName() + TEXT(".uproject");
	FString ProvisionPath;

	// get the provision by popping up the file dialog
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	int32 FilterIndex = -1;
	FString FileTypes = TEXT("Provision Files (*.mobileprovision)|*.mobileprovision");

	if ( DesktopPlatform )
	{
		bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("ImportProvisionDialogTitle", "Import Provision").ToString(),
			FPaths::GetProjectFilePath(),
			TEXT(""),
			FileTypes,
			EFileDialogFlags::None,
			OpenFilenames,
			FilterIndex
			);
	}

	if ( bOpened )
	{
		ProvisionPath = FPaths::ConvertRelativePathToFull(OpenFilenames[0]);

		// see if the provision is already installed
		FString DestName = FPaths::GetBaseFilename(ProvisionPath);
		FString Path;
#if PLATFORM_MAC
		Path = FString::Printf(TEXT("%s/Library/%s/Provisioning Profiles"), *FMacPlatformMisc::GetEnvironmentVariable(TEXT("HOME")), FPlatformMisc::XCodeVersionCompare(16, 0, 0) < 0 ? TEXT("MobileDevice") : TEXT("Developer/Xcode/UserData"));
		FString Destination = FString::Printf(TEXT("\"%s/%s.mobileprovision\""), *Path, *DestName);
		FString Destination2 = FString::Printf(TEXT("\"%s/%s.mobileprovision\""), *Path, FApp::GetProjectName());
#else
		Path = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
		FString Destination = FString::Printf(TEXT("%s\\Apple Computer\\MobileDevice\\Provisioning Profiles\\%s.mobileprovision"), *Path, *DestName);
		FString Destination2 = FString::Printf(TEXT("%s\\Apple Computer\\MobileDevice\\Provisioning Profiles\\%s.mobileprovision"), *Path, FApp::GetProjectName());
#endif
		if (FPaths::FileExists(Destination) || FPaths::FileExists(Destination2))
		{
			FString MessagePrompt = FString::Printf(TEXT("%s mobile provision file already exists.  Do you want to replace this provision?"), *DestName);
			if (FPlatformMisc::MessageBoxExt(EAppMsgType::OkCancel, *MessagePrompt, TEXT("File Exists")) == EAppReturnType::Cancel)
			{
				return FReply::Handled();
			}
		}

		const UIOSRuntimeSettings& Settings = *GetDefault<UIOSRuntimeSettings>();
		FString BundleIdentifier = Settings.BundleIdentifier.Replace(*gProjectNameText, FApp::GetProjectName());
		BundleIdentifier = BundleIdentifier.Replace(TEXT("_"), TEXT(""));
#if PLATFORM_MAC
		FString CmdExe = TEXT("/bin/sh");
		FString ScriptPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Build/BatchFiles/RunDotnet.sh"));
		FString IPPPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/DotNET/IOS/IPhonePackagerCLI.dll"));
		FString CommandLine = FString::Printf(TEXT("\"%s\" \"%s\" Install Engine -project \"%s\" -provision \"%s\" -bundlename \"%s\""), *ScriptPath, *IPPPath, *ProjectPath, *ProvisionPath, *BundleIdentifier);
#else
		FString CmdExe = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/DotNET/IOS/IPhonePackager.exe"));
		FString CommandLine = FString::Printf(TEXT("Install Engine -project \"%s\" -provision \"%s\" -bundlename \"%s\""), *ProjectPath, *ProvisionPath, *BundleIdentifier);
#endif
		IPPProcess = MakeShareable(new FMonitoredProcess(CmdExe, CommandLine, true));
		OutputMessage = TEXT("");
		IPPProcess->OnOutput().BindStatic(&OnOutput);
		IPPProcess->Launch();
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FIOSTargetSettingsCustomization::UpdateStatusDelegate), 10.0f);
		if (ProvisionInfoSwitcher.IsValid())
		{
			ProvisionInfoSwitcher->SetActiveWidgetIndex(1);
		}
		RunningIPPProcess = true;
	}

	return FReply::Handled();
}

FReply FIOSTargetSettingsCustomization::OnGenerateImageClicked(const FString SourceImagePath, const FString TargetImagePath, FIntPoint IconRequiredSize)
{
	if (FPaths::FileExists(*TargetImagePath))
	{
		const EAppReturnType::Type Answer = FMessageDialog::Open(EAppMsgType::YesNo, FText::FromString(TEXT("File already exists. Do you want to overwrite it ?")));
		if (Answer == EAppReturnType::No)
		{
			return FReply::Handled();
		}
	}

	if (ensure(FPaths::FileExists(*SourceImagePath)))
	{
		FUEFreeImageWrapper::FreeImage_Initialise();
		if (!FUEFreeImageWrapper::IsValid())
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("Generate_FreeImageFailure", "The FreeImage library could not be correctly initialized."));
			return FReply::Unhandled();
		}

		FREE_IMAGE_FORMAT FileType = FIF_UNKNOWN;
		FileType = FreeImage_GetFileType(TCHAR_TO_FICHAR(*SourceImagePath), 0);
		if (FileType == FIF_UNKNOWN)
		{
			FileType = FreeImage_GetFIFFromFilename(TCHAR_TO_FICHAR(*SourceImagePath));
			if (FileType == FIF_UNKNOWN)
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("Generate_UnknownFileType", "An unknown filetype error occurred while trying to resize the image."));
				return FReply::Unhandled();
			}
		}

		FIBITMAP* Bitmap = FreeImage_Load(FileType, TCHAR_TO_FICHAR(*SourceImagePath), 0);
		if (Bitmap == nullptr)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("Generate_LoadFailed", "The image file could not be loaded while trying to resize it."));
			return FReply::Unhandled();
		}

		FIBITMAP* RescaledImage;
		FREE_IMAGE_FORMAT FifW;
        if ((RescaledImage = FreeImage_Rescale(Bitmap, IconRequiredSize.X, IconRequiredSize.Y, FREE_IMAGE_FILTER::FILTER_LANCZOS3)) == nullptr ||
                (FifW = FreeImage_GetFIFFromFilename(TCHAR_TO_FICHAR(*TargetImagePath))) == FIF_UNKNOWN ||
                !FreeImage_Save(FifW, RescaledImage, TCHAR_TO_FICHAR(*TargetImagePath), 0))
            {
                FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("Generate_ResizeSaveFailed", "An error occurred while resizing or saving the icon file."));
                return FReply::Unhandled();
            }
        
		return FReply::Handled();
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("Generate_OpenFailed", "The image file could not be found."));
		return FReply::Unhandled();
	}
}

FReply FIOSTargetSettingsCustomization::OnInstallCertificateClicked()
{
	// pass the file to IPP to install
	FString ProjectPath = FPaths::IsProjectFilePathSet() ? FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath()) : FPaths::RootDir() / FApp::GetProjectName() / FApp::GetProjectName() + TEXT(".uproject");
	FString CertPath;

	// get the provision by popping up the file dialog
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	int32 FilterIndex = -1;
	FString FileTypes = TEXT("Code Signing Certificates (*.cer;*.p12)|*.cer;*p12");

	if ( DesktopPlatform )
	{
		bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("ImportCertificateDialogTitle", "Import Certificate").ToString(),
			FPaths::GetProjectFilePath(),
			TEXT(""),
			FileTypes,
			EFileDialogFlags::None,
			OpenFilenames,
			FilterIndex
			);
	}

	if ( bOpened )
	{
		const UIOSRuntimeSettings& Settings = *GetDefault<UIOSRuntimeSettings>();
		FString BundleIdentifier = Settings.BundleIdentifier.Replace(*gProjectNameText, FApp::GetProjectName());
		BundleIdentifier = BundleIdentifier.Replace(TEXT("_"), TEXT(""));
		CertPath = FPaths::ConvertRelativePathToFull(OpenFilenames[0]);
#if PLATFORM_MAC
		FString CmdExe = TEXT("/bin/sh");
		FString ScriptPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Build/BatchFiles/RunDotnet.sh"));
		FString IPPPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/DotNET/IOS/IPhonePackagerCLI.dll"));
		FString CommandLine = FString::Printf(TEXT("\"%s\" \"%s\" Install Engine -project \"%s\" -certificate \"%s\" -bundlename \"%s\""), *ScriptPath, *IPPPath, *ProjectPath, *CertPath, *BundleIdentifier);
#else
		FString CmdExe = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/DotNET/IOS/IPhonePackager.exe"));
		FString CommandLine = FString::Printf(TEXT("Install Engine -project \"%s\" -certificate \"%s\" -bundlename \"%s\""), *ProjectPath, *CertPath, *BundleIdentifier);
#endif
		IPPProcess = MakeShareable(new FMonitoredProcess(CmdExe, CommandLine, false));
		OutputMessage = TEXT("");
		IPPProcess->OnOutput().BindStatic(&OnOutput);
		IPPProcess->Launch();
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FIOSTargetSettingsCustomization::UpdateStatusDelegate), 10.0f);
		if (CertificateInfoSwitcher.IsValid())
		{
			CertificateInfoSwitcher->SetActiveWidgetIndex(1);
		}
		RunningIPPProcess = true;
	}

	return FReply::Handled();
}

FReply FIOSTargetSettingsCustomization::OnCertificateRequestClicked()
{
	// TODO: bring up an open file dialog and then install the provision
	return FReply::Handled();
}

FReply FIOSTargetSettingsCustomization::OnGenerateSSHKey(bool IsPrimary)
{
	// see if the key is already generated
	const UIOSRuntimeSettings& Settings = *GetDefault<UIOSRuntimeSettings>();

	FString RemoteServerAddress;
	FString RemoteServerPort;
	FString RSyncUsername;
	int32	colonIndex;

	if (IsPrimary)
	{
		if (Settings.RemoteServerName.FindChar(':', colonIndex))
		{
			RemoteServerAddress = Settings.RemoteServerName.Left(colonIndex);
			RemoteServerPort = Settings.RemoteServerName.RightChop(colonIndex + 1);
		}
		else
		{
			RemoteServerAddress = Settings.RemoteServerName;
			RemoteServerPort = "22";
		}
		RSyncUsername = Settings.RSyncUsername;
	}
	else
	{
		if (Settings.SecondaryRemoteServerName.FindChar(':', colonIndex))
		{
			RemoteServerAddress = Settings.SecondaryRemoteServerName.Left(colonIndex);
			RemoteServerPort = Settings.SecondaryRemoteServerName.RightChop(colonIndex + 1);
		}
		else
		{
			RemoteServerAddress = Settings.SecondaryRemoteServerName;
			RemoteServerPort = "22";
		}
		RSyncUsername = Settings.SecondaryRSyncUsername;
	}

	FString Path = FPlatformMisc::GetEnvironmentVariable(TEXT("APPDATA"));
	FString Destination = FString::Printf(TEXT("%s\\Unreal Engine\\UnrealBuildTool\\SSHKeys\\%s\\%s\\RemoteToolChainPrivate.key"), *Path, *RemoteServerAddress, *RSyncUsername);
	if (FPaths::FileExists(Destination))
	{
		FString MessagePrompt = FString::Printf(TEXT("An SSH Key already exists.  Do you want to replace this key?"));
		if (FPlatformMisc::MessageBoxExt(EAppMsgType::OkCancel, *MessagePrompt, TEXT("Key Exists")) == EAppReturnType::Cancel)
		{
			return FReply::Handled();
		}
	}

	FString CmdExe = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Build/BatchFiles/MakeAndInstallSSHKey.bat"));
	FString CwRsyncPath = Settings.CwRsyncInstallPath.Path;
	if (CwRsyncPath.IsEmpty() || !FPaths::DirectoryExists(CwRsyncPath))
	{
		// If no user specified directory try the bundled ThirdPartyNotUE directory
		CwRsyncPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Extras\\ThirdPartyNotUE\\cwrsync\\bin"));
	}

	if (!FPaths::DirectoryExists(CwRsyncPath))
	{
		UE_LOGF(LogIOSTargetSettings, Error, "cwRsync is not installed correctly");
	}

	FString CygwinPath = TEXT("/cygdrive/") + FString(Path).Replace(TEXT(":"), TEXT("")).Replace(TEXT("\\"), TEXT("/"));
	FString EnginePath = FPaths::EngineDir();
	FString CommandLine = FString::Printf(TEXT("\"%s/ssh.exe\" %s \"%s\\rsync.exe\" \"%s\" %s \"%s\" \"%s\" \"%s\""),
		*CwRsyncPath,
		*RemoteServerPort,
		*CwRsyncPath,
		*(Settings.RSyncUsername),
		*RemoteServerAddress,
		*Path,
		*CygwinPath,
		*EnginePath);

	OutputMessage = TEXT("");
	IPPProcess = MakeShareable(new FMonitoredProcess(CmdExe, CommandLine, false, false));
	IPPProcess->Launch();
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FIOSTargetSettingsCustomization::UpdateStatusDelegate), 10.0f);
	RunningIPPProcess = true;

	return FReply::Handled();
}

const FSlateBrush* FIOSTargetSettingsCustomization::GetProvisionStatus() const
{
	if( bProvisionInstalled )
	{
		return FAppStyle::GetBrush("Icons.Success");
	}
	else
	{
		return FAppStyle::GetBrush("Icons.Error");
	}
}

const FSlateBrush* FIOSTargetSettingsCustomization::GetCertificateStatus() const
{
	if( bCertificateInstalled )
	{
		return FAppStyle::GetBrush("Icons.Success");
	}
	else
	{
		return FAppStyle::GetBrush("Icons.Error");
	}
}

bool FIOSTargetSettingsCustomization::UpdateStatusDelegate(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FIOSTargetSettingsCustomization_UpdateStatusDelegate);

	if (IPPProcess.IsValid())
	{
		if (IPPProcess->Update())
		{
			return true;
		}
		int RetCode = IPPProcess->GetReturnCode();
		IPPProcess = NULL;
		UpdateStatus();
		UpdateSSHStatus();
	}
	RunningIPPProcess = false;

	return false;
}

TSharedRef<ITableRow> FIOSTargetSettingsCustomization::HandleProvisionListGenerateRow( ProvisionPtr InProvision, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(SProvisionListRow, OwnerTable)
		.Provision(InProvision)
		.ProvisionList(ProvisionList)
		.OnProvisionChanged(this, &FIOSTargetSettingsCustomization::HandleProvisionChanged);
}

void FIOSTargetSettingsCustomization::HandleProvisionChanged(FString Provision)
{
	FText OutText;
	MobileProvisionProperty->GetValueAsFormattedText(OutText);
	if (OutText.ToString() != Provision)
	{
		MobileProvisionProperty->SetValueFromFormattedString(Provision);
	}
	SignCertificateProperty->GetValueAsFormattedText(OutText);
	if (Provision == TEXT("") && OutText.ToString() == TEXT(""))
	{
		bManuallySelected = false;
		FilterLists();
	}
	else if (!bManuallySelected)
	{
		bManuallySelected = true;
		FilterLists();
	}
}

void FIOSTargetSettingsCustomization::HandleCertificateChanged(FString Certificate)
{
	FText OutText;
	SignCertificateProperty->GetValueAsFormattedText(OutText);
	if (OutText.ToString() != Certificate)
	{
		SignCertificateProperty->SetValueFromFormattedString(Certificate);
	}
	MobileProvisionProperty->GetValueAsFormattedText(OutText);
	if (Certificate == TEXT("") && OutText.ToString() == TEXT(""))
	{
		bManuallySelected = false;
		FilterLists();
	}
	else if (!bManuallySelected)
	{
		bManuallySelected = true;
		FilterLists();
	}
}

TSharedRef<ITableRow> FIOSTargetSettingsCustomization::HandleCertificateListGenerateRow( CertificatePtr InCertificate, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(SCertificateListRow, OwnerTable)
		.Certificate(InCertificate)
		.CertificateList(CertificateList)
		.OnCertificateChanged(this, &FIOSTargetSettingsCustomization::HandleCertificateChanged);
}

void FIOSTargetSettingsCustomization::HandleAllProvisionsHyperlinkNavigate( bool AllProvisions )
{
	bShowAllProvisions = AllProvisions;
	FilterLists();
}

void FIOSTargetSettingsCustomization::HandleAllCertificatesHyperlinkNavigate( bool AllCertificates )
{
	bShowAllCertificates = AllCertificates;
	FilterLists();
}

void FIOSTargetSettingsCustomization::FilterLists()
{
	FilteredProvisionList.Reset();
	FilteredCertificateList.Reset();

	for (int Index = 0; Index < ProvisionList->Num(); ++Index)
	{
		if (SelectedProvision.Contains((*ProvisionList)[Index]->Name) && SelectedFile.Contains((*ProvisionList)[Index]->FileName) && !bManuallySelected)
		{
			(*ProvisionList)[Index]->bSelected = true;
		}
		else
		{
			(*ProvisionList)[Index]->bSelected = false;
		}
		if (bShowAllProvisions || (*ProvisionList)[Index]->Status.Contains("VALID"))
		{
			FilteredProvisionList.Add((*ProvisionList)[Index]);
		}
	}

	if (ProvisionList->Num() > 0)
	{
		if (ProvisionInfoSwitcher.IsValid())
		{
			ProvisionInfoSwitcher->SetActiveWidgetIndex(3);
		}
		if (FilteredProvisionList.Num() == 0 && !bShowAllProvisions)
		{
			FilteredProvisionList.Append(*ProvisionList);
		}
	}
	else
	{
		if (ProvisionInfoSwitcher.IsValid())
		{
			ProvisionInfoSwitcher->SetActiveWidgetIndex(2);
		}
	}

	for (int Index = 0; Index < CertificateList->Num(); ++Index)
	{
		if (SelectedCert.Contains((*CertificateList)[Index]->Name) && !bManuallySelected)
		{
			(*CertificateList)[Index]->bSelected = true;
		}
		else
		{
			(*CertificateList)[Index]->bSelected = false;
		}
		if (bShowAllCertificates || (*CertificateList)[Index]->Status.Contains("VALID"))
		{
			FilteredCertificateList.Add((*CertificateList)[Index]);
		}
	}

	if (CertificateList->Num() > 0)
	{
		if (CertificateInfoSwitcher.IsValid())
		{
			CertificateInfoSwitcher->SetActiveWidgetIndex(3);
		}
		if (FilteredCertificateList.Num() == 0 && !bShowAllCertificates)
		{
			FilteredCertificateList.Append(*CertificateList);
		}
	}
	else
	{
		if (CertificateInfoSwitcher.IsValid())
		{
			CertificateInfoSwitcher->SetActiveWidgetIndex(2);
		}
	}

	CertificateListView->RequestListRefresh();
	ProvisionListView->RequestListRefresh();
}

bool FIOSTargetSettingsCustomization::IsImportEnabled() const
{
	return !RunningIPPProcess.Get();
}

bool FIOSTargetSettingsCustomization::IsAutomaticSigningEnabled() const
{
	bool bAutomaticSigning = false;
	AutomaticSigningProperty->GetValue(bAutomaticSigning);
	return bAutomaticSigning;
}

void FIOSTargetSettingsCustomization::OnBundleIdentifierChanged(const FText& NewText, ETextCommit::Type CommitType, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	if(!IsBundleIdentifierValid(NewText.ToString()))
	{
		BundleIdTextBox->SetError( LOCTEXT("NameContainsInvalidCharacters", "Identifier may only contain the characters 0-9, A-Z, a-z, period, hyphen, or [PROJECT_NAME]") );
	}
	else
	{
		BundleIdTextBox->SetError(FText::GetEmpty());

		FText OutText;
		InPropertyHandle->GetValueAsFormattedText(OutText);
		if (OutText.ToString() != NewText.ToString())
		{
			InPropertyHandle->SetValueFromFormattedString( NewText.ToString() );
			FindRequiredFiles();
		}
	}
}

void FIOSTargetSettingsCustomization::OnIOSTeamIDTextChanged(const FText& NewText, ETextCommit::Type CommitType, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	BundleIdTextBox->SetError(FText::GetEmpty());

	FText OutText;
	InPropertyHandle->GetValueAsFormattedText(OutText);
	if (OutText.ToString() != NewText.ToString())
	{
		InPropertyHandle->SetValueFromFormattedString(NewText.ToString());
		FindRequiredFiles();
	}
}

void FIOSTargetSettingsCustomization::OnBundleIdentifierTextChanged(const FText& NewText, ETextCommit::Type CommitType, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	if(!IsBundleIdentifierValid(NewText.ToString()))
	{
		BundleIdTextBox->SetError( LOCTEXT("NameContainsInvalidCharacters", "Identifier may only contain the characters 0-9, A-Z, a-z, period, hyphen, or [PROJECT_NAME]") );
	}
	else
	{
		BundleIdTextBox->SetError(FText::GetEmpty());
	}
}

bool FIOSTargetSettingsCustomization::IsBundleIdentifierValid(const FString& inIdentifier)
{
	for(int32 i = 0; i < inIdentifier.Len(); ++i)
	{
		TCHAR	c = inIdentifier[i];
		
		if(c == '[')
		{
			if(inIdentifier.Find(gProjectNameText, ESearchCase::CaseSensitive, ESearchDir::FromStart, i) != i)
			{
				return false;
			}
			i += gProjectNameText.Len();
		}
		else if((c < '0' || c > '9') && (c < 'a' || c > 'z') && (c < 'A' || c > 'Z') && c != '.' && c != '-')
		{
			return false;
		}
	}

	return true;
}

void FIOSTargetSettingsCustomization::OnRemoteServerChanged(const FText& NewText, ETextCommit::Type CommitType, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	FText OutText;
	InPropertyHandle->GetValueAsFormattedText(OutText);
	if (OutText.ToString() != NewText.ToString())
	{
		InPropertyHandle->SetValueFromFormattedString(NewText.ToString());
		OutputMessage = TEXT("");
		UpdateSSHStatus();
	}
}

FText FIOSTargetSettingsCustomization::GetBundleText(TSharedRef<IPropertyHandle> InPropertyHandle) const
{
	FText OutText;
	InPropertyHandle->GetValueAsFormattedText(OutText);
	return OutText;
}

FText FIOSTargetSettingsCustomization::GetIOSTeamIDText(TSharedRef<IPropertyHandle> InPropertyHandle) const
{
	FText OutText;
	InPropertyHandle->GetValueAsFormattedText(OutText);
	return OutText;
}

TSharedRef<SWidget> FIOSTargetSettingsCustomization::OnGetShaderVersionContent()
{
	FMenuBuilder MenuBuilder(true, NULL);
	
	UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/IOSRuntimeSettings.EIOSMetalShaderStandard"), EFindObjectFlags::ExactClass);
	
	for (int32 i = 0; i < Enum->GetMaxEnumValue(); i++)
	{
		if (Enum->IsValidEnumValue(i) && !Enum->HasMetaData(TEXT("Hidden"), Enum->GetIndexByValue(i)))
		{
			FUIAction ItemAction(FExecuteAction::CreateSP(this, &FIOSTargetSettingsCustomization::SetShaderStandard, i));
			MenuBuilder.AddMenuEntry(Enum->GetDisplayNameTextByValue(i), TAttribute<FText>(), FSlateIcon(), ItemAction);
		}
	}
	
	return MenuBuilder.MakeWidget();
}

FText FIOSTargetSettingsCustomization::GetShaderVersionDesc() const
{
	uint8 EnumValue;
	ShaderVersionPropertyHandle->GetValue(EnumValue);
	
	UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/IOSRuntimeSettings.EIOSMetalShaderStandard"), EFindObjectFlags::ExactClass);
	
	if (EnumValue < Enum->GetMaxEnumValue() && Enum->IsValidEnumValue(EnumValue))
	{
		return Enum->GetDisplayNameTextByValue(EnumValue);
	}
	
	return FText::GetEmpty();
}

TSharedRef<SWidget> FIOSTargetSettingsCustomization::OnGetMinVersionContent()
{
	FMenuBuilder MenuBuilder(true, NULL);

	UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/IOSRuntimeSettings.EIOSVersion"), EFindObjectFlags::ExactClass);

	for (int32 i = 0; i < Enum->GetMaxEnumValue(); i++)
	{
		if (Enum->IsValidEnumValue(i) && !Enum->HasMetaData(TEXT("Hidden"), Enum->GetIndexByValue(i)))
		{
			FUIAction ItemAction(FExecuteAction::CreateSP(this, &FIOSTargetSettingsCustomization::SetMinVersion, i));
			MenuBuilder.AddMenuEntry(Enum->GetDisplayNameTextByValue(i), TAttribute<FText>(), FSlateIcon(), ItemAction);
		}
	}

	return MenuBuilder.MakeWidget();
}

FText FIOSTargetSettingsCustomization::GetMinVersionDesc() const
{
	uint8 EnumValue;
	MinOSPropertyHandle->GetValue(EnumValue);

	UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/IOSRuntimeSettings.EIOSVersion"), EFindObjectFlags::ExactClass);

	if (EnumValue < Enum->GetMaxEnumValue() && Enum->IsValidEnumValue(EnumValue))
	{
		return Enum->GetDisplayNameTextByValue(EnumValue);
	}

	return FText::GetEmpty();
}

void FIOSTargetSettingsCustomization::SetShaderStandard(int32 Value)
{
    FPropertyAccess::Result Res = ShaderVersionPropertyHandle->SetValue((uint8)Value);
    check(Res == FPropertyAccess::Success);

    if (MinOSPropertyHandle.IsValid())
    {
        uint8 IOSVersion = (uint8)EIOSVersion::IOS_Minimum;
        if (MinOSPropertyHandle.IsValid())
        {
            MinOSPropertyHandle->GetValue(IOSVersion);
        }
    }
}

void FIOSTargetSettingsCustomization::UpdateShaderStandardWarning()
{
	// Update the UI
	uint8 EnumValue;
	ShaderVersionPropertyHandle->GetValue(EnumValue);
	SetShaderStandard(EnumValue);
}

void FIOSTargetSettingsCustomization::UpdateOSVersionWarning()
{
    uint8 EnumValue;
    MinOSPropertyHandle->GetValue(EnumValue);

    if (MRTPropertyHandle.IsValid() && ShaderVersionPropertyHandle.IsValid() && MinOSPropertyHandle.IsValid())
	{
		bool bSM5Enabled = false;
		bool bSM6Enabled = false;
		MRTPropertyHandle->GetValue(bSM5Enabled);
		if (SM6PropertyHandle.IsValid())
		{
			SM6PropertyHandle->GetValue(bSM6Enabled);
		}

		if (bSM5Enabled || bSM6Enabled)
		{
			if (EnumValue < (uint8)EIOSVersion::IOS_Minimum)
			{
				SetMinVersion((int32)EIOSVersion::IOS_Minimum);
			}
		}
		else
		{
			FText Message;
			IOSVersionWarningTextBox->SetError(Message);
		}
	}
    
    IOSVersionWarningTextBox->SetError(TEXT(""));


    uint8 ShaderStandard;
    ShaderVersionPropertyHandle->GetValue(ShaderStandard);
    switch (ShaderStandard)
    {
        case (int32)EIOSMetalShaderStandard::IOSMetalSLStandard_Minimum:
        case (int32)EIOSMetalShaderStandard::IOSMetalSLStandard_2_4:
            if (EnumValue < (uint8)EIOSVersion::IOS_15) {IOSVersionWarningTextBox->SetError(TEXT("iOS15 is the Minimum for Metal 2.4")); return;}
            break;
    }
}

void FIOSTargetSettingsCustomization::UpdateMetalMobileWarning()
{
	if (MRTPropertyHandle.IsValid() && ShaderVersionPropertyHandle.IsValid() && MinOSPropertyHandle.IsValid())
	{
		bool bSM5Enabled = false;
		bool bSM6Enabled = false;
		MRTPropertyHandle->GetValue(bSM5Enabled);
		if (SM6PropertyHandle.IsValid())
		{
			SM6PropertyHandle->GetValue(bSM6Enabled);
		}

		if (bSM5Enabled || bSM6Enabled)
		{
			uint8 EnumValue;
			MinOSPropertyHandle->GetValue(EnumValue);
			if (EnumValue < (uint8)EIOSVersion::IOS_Minimum)
			{
				SetMinVersion((int32)EIOSVersion::IOS_Minimum);
			}

			ShaderVersionPropertyHandle->GetValue(EnumValue);
			if (EnumValue < (int32)EIOSMetalShaderStandard::IOSMetalSLStandard_2_4)
			{
				SetShaderStandard((int32)EIOSMetalShaderStandard::IOSMetalSLStandard_Minimum);
			}
		}
		else
		{
			UpdateOSVersionWarning();
			UpdateShaderStandardWarning();
		}
	}
}

void FIOSTargetSettingsCustomization::UpdateGLVersionWarning()
{
	UpdateShaderStandardWarning();
}

void FIOSTargetSettingsCustomization::SetMinVersion(int32 Value)
{
	FPropertyAccess::Result Res = MinOSPropertyHandle->SetValue((uint8)Value);
	check(Res == FPropertyAccess::Success);
}

EVisibility FIOSTargetSettingsCustomization::ShouldShowGenerateButtonForIcon(bool bCannotBeGenerated, const FString ImageToCheck) const
{
	if (!bCannotBeGenerated && FPlatformFileManager::Get().GetPlatformFile().FileExists(*ImageToCheck))
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
