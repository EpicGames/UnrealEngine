// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "IDetailCustomization.h"
#include "ShaderFormatsPropertyDetails.h"
#include "TargetPlatformAudioCustomization.h"
#include "Containers/Ticker.h"

class FMonitoredProcess;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SEditableTextBox;
class SErrorText;

//////////////////////////////////////////////////////////////////////////
// FProvision structure
class FProvision
{
public:
	FString Name;
	FString FileName;
	FString Status;
	bool bDistribution;
	bool bSelected;
	bool bManuallySelected;
};
typedef TSharedPtr<class FProvision> ProvisionPtr;
typedef TSharedPtr<TArray<ProvisionPtr>> ProvisionListPtr;

//////////////////////////////////////////////////////////////////////////
// FCertificate structure
class FCertificate
{
public:
	FString Name;
	FString Status;
	FString Expires;
	bool bSelected;
	bool bManuallySelected;
};
typedef TSharedPtr<class FCertificate> CertificatePtr;
typedef TSharedPtr<TArray<CertificatePtr>> CertificateListPtr;

//////////////////////////////////////////////////////////////////////////
// FIOSTargetSettingsCustomization

class FIOSTargetSettingsCustomization : public IDetailCustomization
{
public:
	~FIOSTargetSettingsCustomization();
	
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface
	
	EVisibility ShouldShowGenerateButtonForIcon(bool bCannotBeGenerated, FString ImageToCheck) const;
	
private:
	TArray<struct FPlatformIconInfo> IconNames;
	TArray<struct FPlatformIconInfo> LaunchImageNames;
	TArray<struct FPlatformIconInfo> TvOSImageNames;
	
	const FString EngineInfoPath;
	const FString GameInfoPath;
	
	const FString EngineGraphicsPath;
	const FString GameGraphicsPath;
	
	const FString TVOSEngineGraphicsPath;
	const FString TVOSGameGraphicsPath;
	
	IDetailLayoutBuilder* SavedLayoutBuilder;
	
	// Is the manifest writable?
	TAttribute<bool> SetupForPlatformAttribute;
	
	bool bProvisionInstalled;
	bool bCertificateInstalled;
	bool bShowAllProvisions;
	bool bShowAllCertificates;
	bool bManuallySelected;
	
	TSharedPtr<FMonitoredProcess> IPPProcess;
	FTSTicker::FDelegateHandle TickerHandle;
	TSharedPtr<TArray<ProvisionPtr>> ProvisionList;
	TArray<ProvisionPtr> FilteredProvisionList;
	TSharedPtr<SListView<ProvisionPtr> > ProvisionListView;
	TSharedPtr<SWidgetSwitcher > ProvisionInfoSwitcher;
	TSharedPtr<TArray<CertificatePtr>> CertificateList;
	TArray<CertificatePtr> FilteredCertificateList;
	TSharedPtr<SListView<CertificatePtr> > CertificateListView;
	TSharedPtr<SWidgetSwitcher > CertificateInfoSwitcher;
	TAttribute<bool> RunningIPPProcess;
	TSharedPtr<IPropertyHandle> MobileProvisionProperty;
	TSharedPtr<IPropertyHandle> SignCertificateProperty;
	TSharedPtr<IPropertyHandle> ShaderVersionPropertyHandle;
	TSharedPtr<IPropertyHandle> MinOSPropertyHandle;
	TSharedPtr<IPropertyHandle> MRTPropertyHandle;
	TSharedPtr<IPropertyHandle> SM6PropertyHandle;
	TSharedPtr<IPropertyHandle> AutomaticSigningProperty;
	
	FString SelectedProvision;
	FString SelectedFile;
	FString SelectedCert;
	
private:
	FIOSTargetSettingsCustomization();
	
	void BuildPListSection(IDetailLayoutBuilder& DetailLayout);
	void BuildIconSection(IDetailLayoutBuilder& DetailLayout);
	void BuildRemoteBuildingSection(IDetailLayoutBuilder& DetailLayout);
	void BuildSecondaryRemoteMacBuildingSection(IDetailLayoutBuilder& DetailLayout);
	
	// Navigates to the plist in explorer or finder
	FReply OpenPlistFolder();
	
	// Copies the setup files for the platform into the project
	void CopySetupFilesIntoProject();
	
	// Builds an image row
	void BuildImageRow(class IDetailLayoutBuilder& DetailLayout, class IDetailCategoryBuilder& Category, const struct FPlatformIconInfo& Info, const FVector2D& MaxDisplaySize, bool bIsTVOS = false);
	
	// Install the provision
	FReply OnInstallProvisionClicked();
	
	// Install the provision
	FReply OnInstallCertificateClicked();
	
	// certificate request
	FReply OnCertificateRequestClicked();
	
	FReply OnGenerateImageClicked(const FString SourceImagePath, const FString TargetImagePath, FIntPoint IconRequiredSize);
	
	// ssh key request
	FReply OnGenerateSSHKey(bool IsPrimary);
	
	// Get the image to display for the provision status
	const FSlateBrush* GetProvisionStatus() const;
	
	// Get the image to display for the certificate status
	const FSlateBrush* GetCertificateStatus() const;
	
	// find the installed certificates and provisions
	void FindRequiredFiles();
	
	// Update the provision status
	void UpdateStatus();
	
	// Update the ssh key status
	void UpdateSSHStatus();
	
	// status tick delay
	bool UpdateStatusDelegate(float delay);
	
	// handle provision list generate row
	TSharedRef<ITableRow> HandleProvisionListGenerateRow( ProvisionPtr Provision, const TSharedRef<STableViewBase>& OwnerTable );
	
	// handle certificate list generate row
	TSharedRef<ITableRow> HandleCertificateListGenerateRow( CertificatePtr Certificate, const TSharedRef<STableViewBase>& OwnerTable );
	
	// handle which set of provisions to view
	void HandleAllProvisionsHyperlinkNavigate( bool AllProvisions );
	
	// handle which set of provisions to view
	void HandleAllCertificatesHyperlinkNavigate( bool AllCertificates );
	
	// filter the lists based on the settings
	void FilterLists();
	
	// returns whether we are importing or not
	bool IsImportEnabled() const;
	
	// returns whether we are importing or not
	bool IsAutomaticSigningEnabled() const;
	
	// updates the bundle identifier if it is valid and checks for a matching provision/certificate
	void OnBundleIdentifierChanged(const FText& NewText, ETextCommit::Type, TSharedRef<IPropertyHandle> InPropertyHandle);
	
	// posts an error if the bundle identifier has become invalid
	void OnBundleIdentifierTextChanged(const FText& NewText, ETextCommit::Type, TSharedRef<IPropertyHandle> InPropertyHandle);
	
	// posts an error if the bundle identifier has become invalid
	void OnIOSTeamIDTextChanged(const FText& NewText, ETextCommit::Type, TSharedRef<IPropertyHandle> InPropertyHandle);
	
	// returns true if the given string is a valid bundle identifier
	bool IsBundleIdentifierValid(const FString& inIdentifier);
	
	// updates the text in the ini file and checks for a valid provision/certificate
	void OnRemoteServerChanged(const FText& NewText, ETextCommit::Type, TSharedRef<IPropertyHandle> InPropertyHandle);
	
	void HandleProvisionChanged(FString Provision);
	
	void HandleCertificateChanged(FString Certificate);
	
	/** Delegate handler to get the list of shader standards */
	TSharedRef<SWidget> OnGetShaderVersionContent();
	
	/** Delegate handler to get the description of the shader standard */
	FText GetShaderVersionDesc() const;
	
	/** Delegate handler to get the list of shader standards */
	TSharedRef<SWidget> OnGetMinVersionContent();
	
	/** Delegate handler to get the description of the shader standard */
	FText GetMinVersionDesc() const;
	
	void SetShaderStandard(int32 Value);
	
	void UpdateShaderStandardWarning();
	
	void UpdateOSVersionWarning();
	
	void UpdateMetalMobileWarning();
	
	void UpdateGLVersionWarning();
	
	void SetMinVersion(int32 Value);
	
	// 
	FText GetBundleText(TSharedRef<IPropertyHandle> InPropertyHandle) const;
	
	//
	FText GetIOSTeamIDText(TSharedRef<IPropertyHandle> InPropertyHandle) const;
	
	/** Module callback for when the iOS settings tab is selected */
	void OnSelect();
	
	TSharedPtr< SEditableTextBox > BundleIdTextBox;
	
	TSharedPtr< SEditableTextBox > IOSTeamIDTextBox;
	
	/** Reference to the shader version property warning text box. */
	TSharedPtr< SErrorText > ShaderVersionWarningTextBox;
	
	/** Reference to the shader version property warning text box. */
	TSharedPtr< SErrorText > IOSVersionWarningTextBox;
	
	/** Reference to the os version property warning text box. */
	TSharedPtr< SErrorText > GLVersionWarningTextBox;
	/** Structure used to manage audio plugin platform settings */
	FAudioPluginWidgetManager AudioPluginWidgetManager;
};
