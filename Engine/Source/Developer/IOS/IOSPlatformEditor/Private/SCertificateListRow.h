// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "IOSTargetSettingsCustomization.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"
#include "SlateOptMacros.h"

DECLARE_DELEGATE_OneParam(FOnCertificateChanged, FString);

/**
 * Implements a row widget for the certificate list view.
 */
class SCertificateListRow
	: public SMultiColumnTableRow<CertificatePtr>
{
public:

	SLATE_BEGIN_ARGS(SCertificateListRow) { }
		SLATE_ARGUMENT(CertificatePtr, Certificate)
		SLATE_ARGUMENT(CertificateListPtr, CertificateList)
		SLATE_EVENT(FOnCertificateChanged, OnCertificateChanged)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The arguments.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		Certificate = InArgs._Certificate;
		CertificateList = InArgs._CertificateList;
		OnCertificateChanged_Handler = InArgs._OnCertificateChanged;
		
		SMultiColumnTableRow<CertificatePtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == TEXT("Selected"))
		{
			return SNew(SCheckBox)
				.IsChecked(this, &SCertificateListRow::HandleChecked)
				.OnCheckStateChanged(this, &SCertificateListRow::HandleCheckStateChanged);
		}
		else if (ColumnName == TEXT("Name"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SCertificateListRow::HandleSelectedColorAndOpacity)
						.Text(this, &SCertificateListRow::HandleNameText)
				];
		}
		else if (ColumnName == TEXT("Status"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SCertificateListRow::HandleStatusTextColorAndOpacity)
						.Text(this, &SCertificateListRow::HandleStatusTextBlockText)
				];
		}
		else if (ColumnName == TEXT("Expires"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SCertificateListRow::HandleSelectedColorAndOpacity)
					.Text(this, &SCertificateListRow::HandleExpiresText)
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	// Callback for getting the text in the 'Name' column.
	FText HandleNameText( ) const
	{
		return FText::FromString(Certificate->Name);
	}

	// Callback for getting the status text.
	FText HandleStatusTextBlockText( ) const
	{
		if (Certificate->Status == TEXT("EXPIRED"))
		{
			return FText::FromString(TEXT("Expired"));
		}
		return FText::FromString(TEXT("Valid"));
	}

	// Callback for getting the status text color.
	FSlateColor HandleStatusTextColorAndOpacity( ) const
	{
		if (Certificate->Status == TEXT("EXPIRED"))
		{
			return FSlateColor(FLinearColor(1.0f, 0.0f, 0.0f));
		}
		else if (Certificate->bSelected)
		{
			return FSlateColor(FLinearColor(0.0f, 1.0f, 0.0f));
		}
		return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f));
	}

	// Callback for getting the status text color.
	FSlateColor HandleSelectedColorAndOpacity( ) const
	{
		if (Certificate->bSelected)
		{
			return FSlateColor(FLinearColor(0.0f, 1.0f, 0.0f));
		}
		return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f));
	}

	// Callback for getting the text in the 'Expires' column.
	FText HandleExpiresText( ) const
	{
		return FText::FromString(Certificate->Expires);
	}

	ECheckBoxState HandleChecked() const
	{
		return Certificate->bManuallySelected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void HandleCheckStateChanged(ECheckBoxState InState)
	{
		Certificate->bManuallySelected = InState == ECheckBoxState::Checked;

		// update the property
		if (OnCertificateChanged_Handler.IsBound())
		{
			OnCertificateChanged_Handler.Execute(Certificate->bManuallySelected ? Certificate->Name : "");
		}

		// disable any other objects
		for (int32 Idx = 0; Idx < CertificateList->Num(); ++Idx)
		{
			if ((*CertificateList)[Idx] != Certificate && (*CertificateList)[Idx]->bManuallySelected)
			{
				(*CertificateList)[Idx]->bManuallySelected = false;
			}
		}
	}

private:

	// Holds the target device service used to populate this row.
	CertificatePtr Certificate;
	CertificateListPtr CertificateList;
	FOnCertificateChanged OnCertificateChanged_Handler;
};
