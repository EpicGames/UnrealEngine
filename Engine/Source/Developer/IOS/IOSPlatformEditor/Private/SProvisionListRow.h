// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Misc/Paths.h"
#include "Brushes/SlateNoResource.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "IOSTargetSettingsCustomization.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"
#include "Brushes/SlateImageBrush.h"
#include "SlateOptMacros.h"

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( FPaths::EngineContentDir() / TEXT("Editor/Slate") / RelativePath + TEXT(".png"), __VA_ARGS__ )

DECLARE_DELEGATE_OneParam(FOnProvisionChanged, FString);

/**
 * Implements a row widget for the provision list view.
 */
class SProvisionListRow
	: public SMultiColumnTableRow<ProvisionPtr>
{
public:

	SLATE_BEGIN_ARGS(SProvisionListRow) { }
		SLATE_ARGUMENT(ProvisionPtr, Provision)
		SLATE_ARGUMENT(ProvisionListPtr, ProvisionList)
		SLATE_EVENT(FOnProvisionChanged, OnProvisionChanged)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The arguments.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		Provision = InArgs._Provision;
		ProvisionList = InArgs._ProvisionList;
		OnProvisionChanged_Handler = InArgs._OnProvisionChanged;

		SMultiColumnTableRow<ProvisionPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);

		/* Set images for various SCheckBox states ... */
		if (!bInitialized)
		{
			ProvisionCheckBoxStyle = FCheckBoxStyle()
				.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
				.SetUncheckedImage( FSlateNoResource() )
				.SetUncheckedHoveredImage( FSlateNoResource() )
				.SetUncheckedPressedImage( FSlateNoResource() )
				.SetCheckedImage( IMAGE_BRUSH( "Automation/Success", FVector2D(16.0f, 16.0f) ) )
				.SetCheckedHoveredImage( IMAGE_BRUSH( "Automation/Success", FVector2D(16.0f, 16.0f), FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
				.SetCheckedPressedImage( IMAGE_BRUSH( "Automation/Success", FVector2D(16.0f, 16.0f) ) )
				.SetUndeterminedImage(FSlateNoResource() )
				.SetUndeterminedHoveredImage( FSlateNoResource() )
				.SetUndeterminedPressedImage( FSlateNoResource() );

			bInitialized = true;
		}
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == TEXT("Selected"))
		{
			return SNew(SCheckBox)
				.IsChecked(this, &SProvisionListRow::HandleChecked)
				.OnCheckStateChanged(this, &SProvisionListRow::HandleCheckStateChanged);
		}
		else if (ColumnName == TEXT("Name"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SProvisionListRow::HandleSelectedColorAndOpacity)
						.Text(this, &SProvisionListRow::HandleNameText)
				];
		}
		else if (ColumnName == TEXT("File"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SProvisionListRow::HandleSelectedColorAndOpacity)
					.Text(this, &SProvisionListRow::HandleFileText)
				];
		}
		else if (ColumnName == TEXT("Status"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SProvisionListRow::HandleStatusTextColorAndOpacity)
						.Text(this, &SProvisionListRow::HandleStatusTextBlockText)
				];
		}
		else if (ColumnName == TEXT("Distribution"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
						.IsChecked(this, &SProvisionListRow::HandleDistribution)
						.Style(&ProvisionCheckBoxStyle)
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION


private:

	// Callback for getting the text in the 'Name' column.
	FText HandleNameText( ) const
	{
		return FText::FromString(Provision->Name);
	}

	// Callback for getting the text in the 'File' column.
	FText HandleFileText( ) const
	{
		return FText::FromString(Provision->FileName);
	}

	// Callback for getting the status text.
	FText HandleStatusTextBlockText( ) const
	{
		if (Provision->Status == TEXT("NO_MATCH"))
		{
			return FText::FromString(TEXT("Identifier Not Matched"));
		}
		else if (Provision->Status == TEXT("NO_CERT"))
		{
			return FText::FromString(TEXT("No Valid Certificate Found"));
		}
		else if (Provision->Status == TEXT("EXPIRED"))
		{
			return FText::FromString(TEXT("Expired"));
		}
		return FText::FromString(TEXT("Valid"));
	}

	// Callback for getting the status text color.
	FSlateColor HandleStatusTextColorAndOpacity( ) const
	{
		if (Provision->Status == TEXT("NO_MATCH"))
		{
			return FSlateColor(FLinearColor(1.0f, 1.0f, 0.0f));
		}
		else if (Provision->Status == TEXT("NO_CERT") || Provision->Status == TEXT("EXPIRED"))
		{
			return FSlateColor(FLinearColor(1.0f, 0.0f, 0.0f));
		}
		else if (Provision->bSelected)
		{
			return FSlateColor(FLinearColor(0.0f, 1.0f, 0.0f));
		}
		return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f));
	}

	// Callback for getting the status text color.
	FSlateColor HandleSelectedColorAndOpacity( ) const
	{
		if (Provision->bSelected)
		{
			return FSlateColor(FLinearColor(0.0f, 1.0f, 0.0f));
		}
		return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f));
	}

	// Callback to determine distribution
	ECheckBoxState HandleDistribution( ) const
	{
		return Provision->bDistribution ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	ECheckBoxState HandleChecked() const
	{
		return Provision->bManuallySelected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void HandleCheckStateChanged(ECheckBoxState InState)
	{
		Provision->bManuallySelected = InState == ECheckBoxState::Checked;

		// update the property
		if (OnProvisionChanged_Handler.IsBound())
		{
			OnProvisionChanged_Handler.Execute(Provision->bManuallySelected ? Provision->FileName : "");
		}

		// disable any other objects
		for (int32 Idx = 0; Idx < ProvisionList->Num(); ++Idx)
		{
			if ((*ProvisionList)[Idx] != Provision && (*ProvisionList)[Idx]->bManuallySelected)
			{
				(*ProvisionList)[Idx]->bManuallySelected = false;
			}
		}
	}

private:

	// Holds the target device service used to populate this row.
	ProvisionPtr Provision;
	ProvisionListPtr ProvisionList;
	FOnProvisionChanged OnProvisionChanged_Handler;

	static bool bInitialized;
	static FCheckBoxStyle ProvisionCheckBoxStyle;
};
