// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IOSTargetPlatform : ModuleRules
{
	public IOSTargetPlatform(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "IOS";
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.IOS);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"TargetPlatform",
				"DesktopPlatform",
			}
		);
	}
}
