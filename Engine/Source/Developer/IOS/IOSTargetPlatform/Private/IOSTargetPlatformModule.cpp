// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/ITargetPlatformModule.h"
#include "Modules/ModuleManager.h"
#include "Common/TargetPlatformBase.h"

/**
 * Module for iOS as a target platform
 */
class FIOSTargetPlatformModule : public ITargetPlatformModule
{
public:

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms) override
	{
	}

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms, TArray<ITargetPlatformSettings*> TargetPlatformSettings, TArray<ITargetPlatformControls*> TargetPlatformControls)
	{
		for (ITargetPlatformControls* TargetPlatformControlsIt : TargetPlatformControls)
		{
			TargetPlatforms.Add(new FTargetPlatformMerged(TargetPlatformControlsIt->GetTargetPlatformSettings(), TargetPlatformControlsIt));
		}
	}
};


IMPLEMENT_MODULE(FIOSTargetPlatformModule, IOSTargetPlatform);
