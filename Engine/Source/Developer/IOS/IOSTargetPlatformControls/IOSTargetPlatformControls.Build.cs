// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IOSTargetPlatformControls : ModuleRules
{
	public IOSTargetPlatformControls(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "IOS";
		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "IOSTPCon";
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.IOS);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"TargetPlatform",
				"DesktopPlatform",
				"LaunchDaemonMessages",
				"Projects",
				"Json",
				"AudioPlatformConfiguration",
				"Sockets",
				"Networking",
				"IOSTargetPlatformSettings"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"MessagingCommon",
				"TargetDeviceServices",
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.Add("Engine");
		}

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicAdditionalLibraries.Add("/System/Library/PrivateFrameworks/MobileDevice.framework/Versions/Current/MobileDevice");
		}
	}
}
