// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSDeviceHelper.h"
#include "IOSTargetPlatformControls.h"
#include "IOSTargetDeviceOutput.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/IProjectManager.h"
#include "Misc/MessageDialog.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#define LOCTEXT_NAMESPACE "MiscIOSMessages"

DEFINE_LOG_CATEGORY_STATIC(LogIOSDeviceHelper, Log, All);

const FString StringifyDeviceConnection(EDeviceConnectionInterface Interface)
{
	switch(Interface)
	{
		case EDeviceConnectionInterface::NoValue:
			return TEXT("NoValue");

		case EDeviceConnectionInterface::Network:
			return TEXT("Network");

		case EDeviceConnectionInterface::USB:
			return TEXT("USB");

		case EDeviceConnectionInterface::Simulator:
			return TEXT("Simulator");

		default:
			UE_LOGF(LogIOSDeviceHelper, Warning, "Unknown EDeviceConnectionInterface type:%d", Interface);
			return TEXT("NoValue");
	}	
}

TArray<FLibIMobileDevice> FIOSDeviceHelper::GetLibIMobileDevices()
{
	FString OutStdOut;
	FString OutStdErr;
	FString LibimobileDeviceId = GetLibImobileDeviceExe("idevice_id");
	int ReturnCode;
	// get the list of devices UDID
	FPlatformProcess::ExecProcess(*LibimobileDeviceId, TEXT(""), &ReturnCode, &OutStdOut, &OutStdErr, NULL, true);
	
	
	TArray<FLibIMobileDevice> ToReturn;
	// separate out each line
	TArray<FString> OngoingDeviceIds;
	
	OutStdOut.ParseIntoArray(OngoingDeviceIds, TEXT("\n"), true);
	TArray<FString> DeviceStrings;
	for (int32 StringIndex = 0; StringIndex < OngoingDeviceIds.Num(); ++StringIndex)
	{
		const FString& DeviceUDID = OngoingDeviceIds[StringIndex];
		EDeviceConnectionInterface OngoingDeviceInterface = EDeviceConnectionInterface::NoValue;
		
		FString OutStdOutInfo;
		FString OutStdErrInfo;
		FString LibimobileDeviceInfo = GetLibImobileDeviceExe("ideviceinfo");
		int ReturnCodeInfo;
		FString Arguments;
		
		if (OngoingDeviceIds[StringIndex].Contains("USB"))
		{
			OngoingDeviceInterface = EDeviceConnectionInterface::USB;
		}
		else if (OngoingDeviceIds[StringIndex].Contains("Network"))
		{
			OngoingDeviceInterface = EDeviceConnectionInterface::Network;
		}
		OngoingDeviceIds[StringIndex].Split(TEXT(" "), &OngoingDeviceIds[StringIndex], nullptr, ESearchCase::CaseSensitive, ESearchDir::FromStart);
		
		if (OngoingDeviceInterface == EDeviceConnectionInterface::USB)
		{
			Arguments = "-u " + DeviceUDID;
		}
		else if (OngoingDeviceInterface == EDeviceConnectionInterface::Network)
		{
			Arguments = "-n -u " + DeviceUDID;
		}
		
		FPlatformProcess::ExecProcess(*LibimobileDeviceInfo, *Arguments, &ReturnCodeInfo, &OutStdOutInfo, &OutStdErrInfo, NULL, true);
		
		FLibIMobileDevice ToAdd;
		
		// ideviceinfo can fail when the connected device is untrusted. It can be "Pairing dialog response pending (-19)", "Invalid HostID (-21)" or "User denied pairing (-18)"
		// the only thing we can do is to make sure the Trust popup is correctly displayed.
		if (OutStdErrInfo.Contains("ERROR: "))
		{
			if (OutStdErrInfo.Contains("Could not connect to lockdownd"))
			{
				// UE_LOGF(LogIOSDeviceHelper, Warning, "Could not pair with connected iOS/tvOS device. Trust this computer by accepting the popup on device.");
				FString LibimobileDevicePair = GetLibImobileDeviceExe("idevicepair");
				FString PairArguments = "-u " + DeviceUDID + " pair";
				FPlatformProcess::ExecProcess(*LibimobileDevicePair, *PairArguments, &ReturnCodeInfo, &OutStdOutInfo, &OutStdErrInfo, NULL, true);
			}
			else
			{
				UE_LOGF(LogIOSDeviceHelper, Warning, "Libimobile call failed : %ls", *OutStdErrInfo);
			}
			OutStdOutInfo.Empty();
			OutStdErrInfo.Empty();
			ToAdd.IsAuthorized = false;
		}
		else
		{
			ToAdd.IsAuthorized = true;
		}
		
		// parse product type and device name
		FString DeviceName;
		OutStdOutInfo.Split(TEXT("DeviceName: "), nullptr, &DeviceName, ESearchCase::CaseSensitive, ESearchDir::FromStart);
		DeviceName.Split(LINE_TERMINATOR, &DeviceName, nullptr, ESearchCase::CaseSensitive, ESearchDir::FromStart);
		if (!ToAdd.IsAuthorized)
		{
			DeviceName = LOCTEXT("IosTvosUnauthorizedDevice", "iOS / tvOS (Unauthorized)").ToString();
		}
		else
		{
			if (OngoingDeviceInterface == EDeviceConnectionInterface::Network)
			{
				DeviceName += " [Wifi]";
			}
		}
		
		FString ProductType;
		OutStdOutInfo.Split(TEXT("ProductType: "), nullptr, &ProductType, ESearchCase::CaseSensitive, ESearchDir::FromStart);
		ProductType.Split(LINE_TERMINATOR, &ProductType, nullptr, ESearchCase::CaseSensitive, ESearchDir::FromStart);
		
		FString OSVersion; // iOS/iPad OS Version
		OutStdOutInfo.Split(TEXT("ProductVersion: "), nullptr, &OSVersion, ESearchCase::CaseSensitive, ESearchDir::FromStart);
		OSVersion.Split(LINE_TERMINATOR, &OSVersion, nullptr, ESearchCase::CaseSensitive, ESearchDir::FromStart);
		
		FString DeviceID = FString::Printf(TEXT("%s@%s"),
										   ProductType.Contains(TEXT("AppleTV")) ? TEXT("TVOS") : TEXT("IOS"),
										   *DeviceUDID);
		
		ToAdd.DeviceID = DeviceID;
		ToAdd.DeviceUDID = DeviceUDID;
		ToAdd.DeviceName = DeviceName;
		ToAdd.DeviceType = ProductType;
		ToAdd.DeviceOSVersion = OSVersion;
		ToAdd.DeviceInterface = OngoingDeviceInterface;
		ToAdd.IsDealtWith = false;
		ToReturn.Add(ToAdd);
	}
	return ToReturn;
}

// Special fake UUID, this needs to be consistent with AppleToolChainSettings.LocalMacUUID
const FString FIOSDeviceHelper::LocalMacDummyUUID = TEXT("10CA18AC-10CA18AC10CA18AC");

// Run iOS app natively on local Mac
static TArray<FLibIMobileDevice> GetLocalMacDevices()
{
	TArray<FLibIMobileDevice> ToReturn;

	FLibIMobileDevice ToAdd;
	ToAdd.IsAuthorized = true;
	ToAdd.DeviceOSVersion = TEXT("99.99");
	ToAdd.DeviceUDID = FIOSDeviceHelper::LocalMacDummyUUID;
	ToAdd.DeviceType = TEXT("iOSAppOnMac");
	bool bSupportsIPad = true;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsIPad"), bSupportsIPad, GEngineIni);
	if (bSupportsIPad)	// Same logic as Xcode, iPad takes priority
	{
		ToAdd.DeviceName = TEXT("My Mac (Designed for iPad)");
	}
	else
	{
		ToAdd.DeviceName = TEXT("My Mac (Designed for iPhone)");
	}
	ToAdd.DeviceID = FString::Printf(TEXT("IOS@%s"), *ToAdd.DeviceUDID);
	ToAdd.DeviceInterface = EDeviceConnectionInterface::NoValue;

	ToReturn.Add(ToAdd);
	
	return ToReturn;
}

class FIOSDevice
{
public:
	FIOSDevice(FString InID, FString InName, EDeviceConnectionInterface InConnectionType)
	: UDID(InID)
	, Name(InName)
	, ConnectionType(InConnectionType)
	{
	}
	
	~FIOSDevice()
	{
	}
	
	FString SerialNumber() const
	{
		return UDID;
	}
	
	EDeviceConnectionInterface ConnectionInterface() const
	{
		return ConnectionType;
	}
	
private:
	FString UDID;
	FString Name;
	EDeviceConnectionInterface ConnectionType;
};

/**
 * Delegate type for devices being connected or disconnected from the machine
 *
 * The first parameter is newly added or removed device
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FDeviceNotification, void*)

class FDeviceQueryTask
: public FRunnable
{
public:
	FDeviceQueryTask()
	: Stopping(false)
	, bCheckDevices(true)
	, RetryQuery(5)
	{}
	
	virtual bool Init() override
	{
		return true;
	}
	
	virtual uint32 Run() override
	{
		int32 RecheckRateInt = 10;	// 10s default, which should match "AppleRescanRate" in BaseEditor.ini
		GConfig->GetInt(TEXT("DeviceDiscovery"), TEXT("AppleRescanRate"), RecheckRateInt, GEditorIni);
		while (!Stopping)
		{
			if (IsEngineExitRequested())
			{
				break;
			}
			if (GetTargetPlatformManager())
			{
				FString OutTutorialPath;
				const ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(TEXT("IOS"));
				if (Platform)
				{
					if (Platform->IsSdkInstalled(false, OutTutorialPath))
					{
						break;
					}
				}
				Enable(false);
				return 0;
			}
			else
			{
				FPlatformProcess::Sleep(1.0f);
			}
		}

		bCheckDevices = true;
		float RecheckRate = static_cast<float>(RecheckRateInt);
		double StartTime = FPlatformTime::Seconds();
		while (!Stopping)
		{
			if (IsEngineExitRequested())
			{
				break;
			}
			if (bCheckDevices)
			{
#if WITH_EDITOR
				// BHP - Turning off device check to prevent it from interfering with packaging
				if (!IsRunningCommandlet())
#endif
				{
					QueryDevices();
				}
			}
			
			float TimeDelta = FPlatformTime::Seconds() - StartTime;
			FPlatformProcess::Sleep(FMath::Clamp((RecheckRate-TimeDelta), 0, RecheckRate));
			StartTime = FPlatformTime::Seconds();
		}
		
		return 0;
	}
	
	virtual void Stop() override
	{
		Stopping = true;
	}
	
	virtual void Exit() override
	{}
	
	FDeviceNotification& OnDeviceNotification()
	{
		return DeviceNotification;
	}
	
	void Enable(bool bInCheckDevices)
	{
		bCheckDevices = bInCheckDevices;
	}
	
private:
	
	void NotifyDeviceChange(FLibIMobileDevice& Device, bool bAdd)
	{
		FDeviceNotificationCallbackInformation CallbackInfo;
		
		if (bAdd)
		{
			CallbackInfo.DeviceID = Device.DeviceID;
			CallbackInfo.DeviceName = Device.DeviceName;
			CallbackInfo.DeviceUDID = Device.DeviceUDID;
			CallbackInfo.DeviceInterface = Device.DeviceInterface;
			CallbackInfo.ProductType = Device.DeviceType;
			CallbackInfo.DeviceOSVersion = Device.DeviceOSVersion;
			CallbackInfo.msgType = 1;
			CallbackInfo.IsAuthorized = Device.IsAuthorized;
		}
		else
		{
			CallbackInfo.DeviceID = Device.DeviceID;
			CallbackInfo.DeviceName = Device.DeviceName;
			CallbackInfo.DeviceUDID = Device.DeviceUDID;
			CallbackInfo.DeviceInterface = Device.DeviceInterface;
			CallbackInfo.msgType = 2;
			DeviceNotification.Broadcast(&CallbackInfo);
			
		}
		DeviceNotification.Broadcast(&CallbackInfo);
	}
	
	void QueryDevices()
	{
		bool HasOtherDevices = false;
		bool HasDevices = true;
		
#if PLATFORM_MAC_ARM64
		// We can run iOS app natively on arm64 Mac, add a special dummy device
		TArray<FLibIMobileDevice> ParsedDevices = GetLocalMacDevices();
		HasOtherDevices = true;
#else
		TArray<FLibIMobileDevice> ParsedDevices;
#endif

		FString LibimobileDeviceId = GetLibImobileDeviceExe("idevice_id");
		if (LibimobileDeviceId.Len() == 0)
		{
			UE_LOGF(LogIOSDeviceHelper, Warning, "idevice_id (iOS device detection) executable missing. Turning off iOS/tvOS device detection.");
			HasDevices = false;
			Enable(false);
			if (!HasOtherDevices)
			{
				return;
			}
		}

		// get the list of devices UDID
		if (HasDevices)
		{
			FString OutStdOut;
			FString OutStdErr;
			int ReturnCode;
			FPlatformProcess::ExecProcess(*LibimobileDeviceId, TEXT(""), &ReturnCode, &OutStdOut, &OutStdErr, NULL, true);
			if (OutStdOut.Len() == 0)
			{
				RetryQuery--;
				if (RetryQuery < 0)
				{
					UE_LOGF(LogIOSDeviceHelper, Verbose, "IOS device listing is disabled for 1 minute (too many failed attempts)!");
					Enable(false);
				}
				for (FLibIMobileDevice device : CachedDevices)
				{
					NotifyDeviceChange(device, false);
				}
				CachedDevices.Empty();
				HasDevices = false;
				if (!HasOtherDevices)
				{
					return;
				}
			}
			RetryQuery = 5;
		}

		if (HasDevices)
		{
			ParsedDevices.Append(FIOSDeviceHelper::GetLibIMobileDevices());
		}

		for (int32 Index = 0; Index < ParsedDevices.Num(); ++Index)
		{
			FLibIMobileDevice *Found = CachedDevices.FindByPredicate(
				[&](FLibIMobileDevice Element)
				{
					return (Element.DeviceUDID == ParsedDevices[Index].DeviceUDID &&
							Element.DeviceInterface == ParsedDevices[Index].DeviceInterface);
				});
			if (Found != nullptr)
			{
				if (Found->IsAuthorized != ParsedDevices[Index].IsAuthorized)
				{
					NotifyDeviceChange(ParsedDevices[Index], false);
					NotifyDeviceChange(ParsedDevices[Index], true);
				}
				Found->IsDealtWith = true;
			}
			else
			{
				NotifyDeviceChange(ParsedDevices[Index], true);
			}
			
		}
		
		for (int32 Index = 0; Index < CachedDevices.Num(); ++Index)
		{
			if (!CachedDevices[Index].IsDealtWith)
			{
				NotifyDeviceChange(CachedDevices[Index], false);
			}
		}
		
		CachedDevices.Empty();
		CachedDevices = ParsedDevices;
	}
	
	bool Stopping;
	bool bCheckDevices;
	int RetryQuery;
	TArray<FLibIMobileDevice> CachedDevices;
	FDeviceNotification DeviceNotification;
};

/* FIOSDeviceHelper structors
 *****************************************************************************/
static TMap<FIOSDevice*, FIOSLaunchDaemonPong> ConnectedDevices;
static FDeviceQueryTask* QueryTask = NULL;
static FRunnableThread* QueryThread = NULL;
static TArray<FDeviceNotificationCallbackInformation> NotificationMessages;
static FTickerDelegate TickDelegate;

bool FIOSDeviceHelper::MessageTickDelegate(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FIOSDeviceHelper_MessageTickDelegate);
	
	for (int Index = 0; Index < NotificationMessages.Num(); ++Index)
	{
		FDeviceNotificationCallbackInformation cbi = NotificationMessages[Index];
		FIOSDeviceHelper::DeviceCallback(&cbi);
	}
	NotificationMessages.Empty();
	
	return true;
}

void FIOSDeviceHelper::Initialize(bool bIsTVOS)
{
	if(!bIsTVOS)
	{
		// add the message pump
		TickDelegate = FTickerDelegate::CreateStatic(MessageTickDelegate);
		FTSTicker::GetCoreTicker().AddTicker(TickDelegate, 5.0f);
		
		// kick off a thread to query for connected devices
		QueryTask = new FDeviceQueryTask();
		QueryTask->OnDeviceNotification().AddStatic(FIOSDeviceHelper::DeviceCallback);
		
		static int32 QueryTaskCount = 1;
		if (QueryTaskCount == 1)
		{
			// create the socket subsystem (loadmodule in game thread)
			ISocketSubsystem* SSS = ISocketSubsystem::Get();
			QueryThread = FRunnableThread::Create(QueryTask, *FString::Printf(TEXT("FIOSDeviceHelper.QueryTask_%d"), QueryTaskCount++), 128 * 1024, TPri_Normal);
		}
	}
}

void FIOSDeviceHelper::DeviceCallback(void* CallbackInfo)
{
	struct FDeviceNotificationCallbackInformation* cbi = (FDeviceNotificationCallbackInformation*)CallbackInfo;
	
	if (!IsInGameThread())
	{
		NotificationMessages.Add(*cbi);
	}
	else
	{
		switch(cbi->msgType)
		{
			case 1:
				FIOSDeviceHelper::DoDeviceConnect(CallbackInfo);
				break;
				
			case 2:
				FIOSDeviceHelper::DoDeviceDisconnect(CallbackInfo);
				break;
		}
	}
}

void FIOSDeviceHelper::DoDeviceConnect(void* CallbackInfo)
{
	// connect to the device
	struct FDeviceNotificationCallbackInformation* cbi = (FDeviceNotificationCallbackInformation*)CallbackInfo;
	FIOSDevice* Device = new FIOSDevice(cbi->DeviceUDID, cbi->DeviceName, cbi->DeviceInterface);
	
	// fire the event
	FIOSLaunchDaemonPong Event;
	Event.DeviceID = cbi->DeviceID;
	Event.DeviceUDID = cbi->DeviceUDID;
	Event.DeviceName = cbi->DeviceName;
	Event.DeviceType = cbi->ProductType;
	Event.DeviceOSVersion = cbi->DeviceOSVersion;
	Event.DeviceModelId = cbi->ProductType;
	Event.DeviceConnectionType = StringifyDeviceConnection(cbi->DeviceInterface);
	Event.bIsAuthorized = cbi->IsAuthorized;
	Event.bCanReboot = false;
	Event.bCanPowerOn = false;
	Event.bCanPowerOff = false;
	FIOSDeviceHelper::OnDeviceConnected().Broadcast(Event);
	
	// add to the device list
	ConnectedDevices.Add(Device, Event);
}

void FIOSDeviceHelper::DoDeviceDisconnect(void* CallbackInfo)
{
	struct FDeviceNotificationCallbackInformation* cbi = (FDeviceNotificationCallbackInformation*)CallbackInfo;
	FIOSDevice* device = NULL;
	for (auto DeviceIterator = ConnectedDevices.CreateIterator(); DeviceIterator; ++DeviceIterator)
	{
		if (DeviceIterator.Key()->SerialNumber() == cbi->DeviceUDID &&
			DeviceIterator.Key()->ConnectionInterface() == cbi->DeviceInterface)
		{
			device = DeviceIterator.Key();
			break;
		}
	}
	
	if (device != NULL)
	{
		// extract the device id from the connected list
		FIOSLaunchDaemonPong Event = ConnectedDevices.FindAndRemoveChecked(device);
		
		// fire the event
		FIOSDeviceHelper::OnDeviceDisconnected().Broadcast(Event);
		
		// delete the device
		delete device;
	}
}

bool FIOSDeviceHelper::InstallIPAOnDevice(const FTargetDeviceId& DeviceId, const FString& IPAPath)
{
	return false;
}

void FIOSDeviceHelper::EnableDeviceCheck(bool OnOff)
{
	QueryTask->Enable(OnOff);
}

#undef LOCTEXT_NAMESPACE
