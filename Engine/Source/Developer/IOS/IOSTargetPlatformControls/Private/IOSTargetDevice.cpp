// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSTargetDevice.h"

#include "HAL/PlatformProcess.h"
#include "IOSMessageProtocol.h"
#include "Interfaces/ITargetPlatformControls.h"
#include "Async/Async.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "IOSTargetDeviceOutput.h"
#if PLATFORM_WINDOWS
// for windows mutex
#include "Windows/AllowWindowsPlatformTypes.h"
#endif // #if PLATFORM_WINDOWS

FIOSTargetDevice::FIOSTargetDevice(const ITargetPlatformControls& InTargetPlatformControls)
	: TargetPlatformControls(InTargetPlatformControls)
	, DeviceEndpoint()
	, AppId()
	, bCanReboot(false)
	, bCanPowerOn(false)
	, bCanPowerOff(false)
	, DeviceType(ETargetDeviceTypes::Indeterminate)
	, DeviceModelId(TEXT(""))
{
	DeviceId = FTargetDeviceId(TargetPlatformControls.PlatformName(), FPlatformProcess::ComputerName());
	DeviceName = FPlatformProcess::ComputerName();
	MessageEndpoint = FMessageEndpoint::Builder("FIOSTargetDevice").Build();
	DeviceConnectionType = ETargetDeviceConnectionTypes::USB;
}


bool FIOSTargetDevice::Connect()
{
	// @todo zombie - Probably need to write a specific ConnectTo(IpAddr) function for setting up a RemoteEndpoint for talking to the Daemon
	// Returning true since, if this exists, a device exists.

	return true;
}

void FIOSTargetDevice::Disconnect()
{
}

int32 FIOSTargetDevice::GetProcessSnapshot(TArray<FTargetDeviceProcessInfo>& OutProcessInfos)
{
	return 0;
}

ETargetDeviceTypes FIOSTargetDevice::GetDeviceType() const
{
	return DeviceType;
}

ETargetDeviceConnectionTypes FIOSTargetDevice::GetDeviceConnectionType() const
{
	return DeviceConnectionType;
}

FTargetDeviceId FIOSTargetDevice::GetId() const
{
	return DeviceId;
}

FString FIOSTargetDevice::GetName() const
{
	return DeviceName;
}

FString FIOSTargetDevice::GetOperatingSystemName()
{
	return TargetPlatformControls.PlatformName();
}

FString FIOSTargetDevice::GetModelId() const
{
	return DeviceModelId;
}

FString FIOSTargetDevice::GetOSVersion() const
{
	return DeviceOSVersion;
}

const class ITargetPlatformSettings& FIOSTargetDevice::GetPlatformSettings() const
{
	return *(TargetPlatformControls.GetTargetPlatformSettings());
}
const class ITargetPlatformControls& FIOSTargetDevice::GetPlatformControls() const
{
	return TargetPlatformControls;
}

bool FIOSTargetDevice::IsConnected()
{
	return true;
}

bool FIOSTargetDevice::IsDefault() const
{
	return true;
}

bool FIOSTargetDevice::PowerOff(bool Force)
{
	// @todo zombie - Supported by the Daemon?

	return false;
}

bool FIOSTargetDevice::PowerOn()
{
	// @todo zombie - Supported by the Daemon?

	return false;
}

bool FIOSTargetDevice::Reboot(bool bReconnect)
{
	// @todo zombie - Supported by the Daemon?

	return false;
}

bool FIOSTargetDevice::SupportsFeature(ETargetDeviceFeatures Feature) const
{
	switch (Feature)
	{
	case ETargetDeviceFeatures::Reboot:
		return bCanReboot;

	case ETargetDeviceFeatures::PowerOn:
		return bCanPowerOn;

	case ETargetDeviceFeatures::PowerOff:
		return bCanPowerOff;

	case ETargetDeviceFeatures::ProcessSnapshot:
		return false;

	default:
		return false;
	}
}

bool FIOSTargetDevice::TerminateProcess(const int64 ProcessId)
{
	return false;
}

void FIOSTargetDevice::SetUserCredentials(const FString& UserName, const FString& UserPassword)
{
}

bool FIOSTargetDevice::GetUserCredentials(FString& OutUserName, FString& OutUserPassword)
{
	return false;
}

inline void FIOSTargetDevice::ExecuteConsoleCommand(const FString& ExecCommand) const
{
	FString OutStdOut;
	FString OutStdErr;
	FString Exe = GetLibImobileDeviceExe("itcpconnect");
	FString Params = FString::Printf(TEXT(" -u %s 8888"), *DeviceId.GetDeviceName());

	void* StdInPipe_Read = nullptr;
	void* StdInPipe_Write = nullptr;

	verify(FPlatformProcess::CreatePipe(StdInPipe_Read, StdInPipe_Write, true));

	FProcHandle ProcHandle = FPlatformProcess::CreateProc(*Exe, *Params, true, false, false, nullptr, 0, nullptr, nullptr, StdInPipe_Read);
	if (ProcHandle.IsValid())
	{
		FPlatformProcess::WritePipe(StdInPipe_Write, ExecCommand, nullptr);
		FPlatformProcess::ClosePipe(StdInPipe_Read, StdInPipe_Write);
		FPlatformProcess::WaitForProc(ProcHandle);
	}
	else
	{
		FPlatformProcess::ClosePipe(StdInPipe_Read, StdInPipe_Write);
	}
}

inline ITargetDeviceOutputPtr FIOSTargetDevice::CreateDeviceOutputRouter(FOutputDevice* Output) const
{
	FIOSTargetDeviceOutputPtr DeviceOutputPtr = MakeShareable(new FIOSTargetDeviceOutput());
	if (DeviceOutputPtr->Init(*this, Output))
	{
		return DeviceOutputPtr;
	}

	return nullptr;
}

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif // #if PLATFORM_WINDOWS