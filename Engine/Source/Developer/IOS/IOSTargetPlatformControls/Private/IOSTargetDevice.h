// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "Interfaces/ITargetDevice.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Containers/Queue.h"

class FMessageEndpoint;
class ITargetPlatformControls;


/** Type definition for shared pointers to instances of FIOSTargetDevice. */
typedef TSharedPtr<class FIOSTargetDevice, ESPMode::ThreadSafe> FIOSTargetDevicePtr;

/** Type definition for shared references to instances of FIOSTargetDevice. */
typedef TSharedRef<class FIOSTargetDevice, ESPMode::ThreadSafe> FIOSTargetDeviceRef;

/** Type definition for shared references to instances of FIOSTargetDeviceOutput. */
typedef TSharedPtr<class FIOSTargetDeviceOutput, ESPMode::ThreadSafe> FIOSTargetDeviceOutputPtr;

/**
 * Handles the communication to the Deployment Server over TCP (will start the DeploymentServer if no instance is found running)
 */
class FTcpDSCommander : FRunnable
{
public:
    
    /**
     * Creates and initializes a new instance.
     *
     */
    FTcpDSCommander(const uint8* Data, int32 Count, TQueue<FString>& InOutputQueue);
    
    /** Virtual destructor. */
    virtual ~FTcpDSCommander();

	/** Check if DeploymentServer mutex is active*/
	static bool IsDSRunning();
    
public:
    
    //~ FRunnable interface
    
    virtual void Exit() override;
    virtual bool Init() override;
    virtual uint32 Run() override;
    virtual void Stop() override;
    
    inline bool IsValid()
    {
        return (Thread != nullptr);
    }
    inline bool IsStopped()
    {
        return bStopped;
    }
    inline bool WasSuccess()
    {
        return bIsSuccess;
    }
	inline bool IsSystemError()
	{
		return bIsSystemError;
	}
    
private:
    
    inline bool StartDSProcess();
    
    /** For the thread */
    bool bStopping;
    bool bStopped;
    bool bIsSuccess;
	bool bIsSystemError; ///< Deployment server was not able to start, or connection to it could not be made
    
    /** */
    class FSocket* DSSocket;
    
    /** Holds the thread object. */
    FRunnableThread* Thread;
    
	TQueue<FString>& OutputQueue;
    uint8* DSCommand;
    int32 DSCommandLen;
    double LastActivity;
};

/**
 * Implements an iOS target device.
 */
class FIOSTargetDevice
	: public ITargetDevice
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InTargetPlatformControls The target platform that owns the device.
	 */
	FIOSTargetDevice(const ITargetPlatformControls& InTargetPlatformControls);

public:

	//~ ITargetDevice interface

	virtual bool Connect() override;
	virtual void Disconnect() override;
	virtual int32 GetProcessSnapshot(TArray<FTargetDeviceProcessInfo>& OutProcessInfos) override;
	virtual ETargetDeviceTypes GetDeviceType() const override;
	virtual ETargetDeviceConnectionTypes GetDeviceConnectionType() const override;
	virtual FTargetDeviceId GetId() const override;
	virtual FString GetName() const override;
	virtual FString GetOperatingSystemName() override;
	virtual FString GetModelId() const override;
	virtual FString GetOSVersion() const override;
	virtual const class ITargetPlatformSettings& GetPlatformSettings() const override;
	virtual const class ITargetPlatformControls& GetPlatformControls() const override;
	virtual bool IsConnected() override;
	virtual bool IsDefault() const override;
	virtual bool PowerOff(bool Force) override;
	virtual bool PowerOn() override;
	virtual bool IsAuthorized() const override { return bIsDeviceAuthorized; }
	virtual bool Reboot(bool bReconnect = false) override;
	virtual bool SupportsFeature(ETargetDeviceFeatures Feature) const;
	virtual bool TerminateProcess(const int64 ProcessId) override;
	virtual void SetUserCredentials(const FString& UserName, const FString& UserPassword) override;
	virtual bool GetUserCredentials(FString& OutUserName, FString& OutUserPassword) override;
	virtual void ExecuteConsoleCommand(const FString& ExecCommand) const override;
	virtual ITargetDeviceOutputPtr CreateDeviceOutputRouter(FOutputDevice* Output) const override;

public:

	/** Timeout check for removing stale devices */
	FDateTime LastPinged;

private:

	/** The current status of this device. */
//	ETargetDeviceStatus::Type Status;

/** Holds a reference to the device's target platform controls. */
	const ITargetPlatformControls& TargetPlatformControls;

	/** Contains the address of the remote device */
	FMessageAddress DeviceEndpoint;

	/** MessageEndpoint for communicating with remote device */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Contains the current AppID/GameName for Deployment/launching. */
	FString AppId;

	/** Contains the build configuration of the app to deploy */
	EBuildConfiguration BuildConfiguration;

	/** Lets us know whether the thing is a sim device or a physical device. */
	bool bIsSimulated;

private:

	/** Remote rebootable */
	bool bCanReboot;

	/** Remote bootable */
	bool bCanPowerOn;

	/** Remote shutdown-able */
	bool bCanPowerOff;

	/** Id of device */
	FTargetDeviceId DeviceId;

	/** Name of device */
	FString DeviceName;

	// Holds a flag indicating whether the device is USB / OTA comms authorized
	bool bIsDeviceAuthorized;

	/** Type of device */
	ETargetDeviceTypes DeviceType;

	/** The specific model identifier of the device */
	FString DeviceModelId;

	/** The iOS/tvOS/iPadOS OS version */
	FString DeviceOSVersion;

	/** Type of device connection (USB or Wifi) */
	ETargetDeviceConnectionTypes DeviceConnectionType;

public:

	void SetFeature(ETargetDeviceFeatures InFeature, bool bFlag)
	{
		if (InFeature == ETargetDeviceFeatures::Reboot)
		{
			bCanReboot = bFlag;
		}
		else if (InFeature == ETargetDeviceFeatures::PowerOn)
		{
			bCanPowerOn = bFlag;
		}
		else if (InFeature == ETargetDeviceFeatures::PowerOff)
		{
			bCanPowerOff = bFlag;
		}
	}

	/** Sets device id */
	void SetDeviceId(const FTargetDeviceId InDeviceId)
	{
		DeviceId = InDeviceId;
	}

	/** Sets the name of the device */
	void SetDeviceName(const FString InDeviceName)
	{
		DeviceName = InDeviceName;
	}

	/** Sets the modelId of the device */
	void SetModelId(const FString InModelId)
	{
		DeviceModelId = InModelId;
	}

	/** Sets the OS version of the device */
	void SetOSVersion(const FString InOSVersion)
	{
		DeviceOSVersion = InOSVersion;
	}

	/**
	 * Sets the device's authorization state.
	 *
	 * @param bInIsAuthorized - Whether the device is authorized for USB communications.
	 */
	void SetAuthorized(bool bInIsAuthorized)
	{
		bIsDeviceAuthorized = bInIsAuthorized;
	}

	/** Sets the type of the device */
	void SetDeviceType(const FString InDeviceTypeString)
	{
		if (InDeviceTypeString.Contains(TEXT("Browser")))
		{
			DeviceType = ETargetDeviceTypes::Browser;
		}
		else if (InDeviceTypeString.Contains(TEXT("Console")))
		{
			DeviceType = ETargetDeviceTypes::Console;
		}
		else if (InDeviceTypeString.Contains(TEXT("Phone")))
		{
			DeviceType = ETargetDeviceTypes::Phone;
		}
		else if (InDeviceTypeString.Contains(TEXT("Tablet")))
		{
			DeviceType = ETargetDeviceTypes::Tablet;
		}
		else if (InDeviceTypeString.Contains(TEXT("iPad")))
		{
			DeviceType = ETargetDeviceTypes::Tablet;
		}
		else
		{
			DeviceType = ETargetDeviceTypes::Indeterminate;
		}
	}

	/** Sets the connection type (usb/wifi) of the device */
	void SetDeviceConnectionType(const FString InDeviceConnectionTypeString)
	{
		if (InDeviceConnectionTypeString == TEXT("Network"))
		{
			DeviceConnectionType = ETargetDeviceConnectionTypes::Wifi;
		}
		else if (InDeviceConnectionTypeString == TEXT("USB"))
		{
			DeviceConnectionType = ETargetDeviceConnectionTypes::USB;
		}
		else if (InDeviceConnectionTypeString == TEXT("Simulator"))
		{
			DeviceConnectionType = ETargetDeviceConnectionTypes::Simulator;
		}
		else
		{
			DeviceConnectionType = ETargetDeviceConnectionTypes::Unknown;
		}
	}

	void SetDeviceEndpoint(const FMessageAddress& DeviceAddress)
	{
		DeviceEndpoint = DeviceAddress;
	}

	void SetAppId(const FString& GameName)
	{
		AppId = GameName;
	}

	void SetAppConfiguration(EBuildConfiguration Configuration)
	{
		BuildConfiguration = Configuration;
	}

	void SetIsSimulated(bool IsSimulated)
	{
		bIsSimulated = IsSimulated;
	}
};
