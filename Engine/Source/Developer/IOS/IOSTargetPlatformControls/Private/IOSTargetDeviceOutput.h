// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/ITargetDeviceOutput.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/ThreadSafeCounter.h"

class FIOSTargetDevice;

static FString GetLibImobileDeviceExe(const FString& ExeName)
{
	FString ToReturn;
#if PLATFORM_WINDOWS
	ToReturn = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Extras/ThirdPartyNotUE/libimobiledevice/x64/"));
#elif PLATFORM_MAC
	ToReturn = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Extras/ThirdPartyNotUE/libimobiledevice/Mac/"));
#else
	UE_LOGF(LogIOSDeviceHelper, Error, "The current platform is unsupported by Libimobile library.");
#endif
	ToReturn += ExeName;
#if PLATFORM_WINDOWS
	ToReturn += TEXT(".exe");
#endif
	return FPaths::FileExists(ToReturn) ? ToReturn : TEXT("");
}

class FIOSDeviceOutputReaderRunnable : public FRunnable
{
public:
	FIOSDeviceOutputReaderRunnable(const FString& InDeviceUDID, FOutputDevice* Output);
	
	// FRunnable interface.
	virtual bool Init(void) override;
	virtual void Exit(void) override; 
	virtual void Stop(void) override;
	virtual uint32 Run(void) override;

private:
	bool StartSyslogProcess(void);

	// > 0 if we've been asked to abort work in progress at the next opportunity
	FThreadSafeCounter	StopTaskCounter;
	
	FString				DeviceUDID;
	FOutputDevice*		Output;
	void*				SyslogReadPipe;
	void*				SyslogWritePipe;
	FProcHandle			SyslogProcHandle;
};

/**
 * Implements a IOS target device output.
 */
class FIOSTargetDeviceOutput : public ITargetDeviceOutput
{
public:
	bool Init(const FIOSTargetDevice& TargetDevice, FOutputDevice* Output);
	
private:
	TUniquePtr<FRunnableThread>						DeviceOutputThread;
	FString											DeviceName;
};

#include "IOSTargetDeviceOutput.inl"
