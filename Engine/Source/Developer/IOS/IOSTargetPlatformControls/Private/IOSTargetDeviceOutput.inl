// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "CoreFwd.h"
#include "IOSTargetDevice.h"
#include "Logging/LogMacros.h"
#include "Common/TcpSocketBuilder.h"
#include "string.h"

class FIOSDeviceOutputReaderRunnable;
class FIOSTargetDevice;
class FIOSTargetDeviceOutput;

inline FIOSDeviceOutputReaderRunnable::FIOSDeviceOutputReaderRunnable(const FString& InDeviceUDID, FOutputDevice* InOutput)
	: StopTaskCounter(0)
	, DeviceUDID(InDeviceUDID)
	, Output(InOutput)
	, SyslogReadPipe(nullptr)
	, SyslogWritePipe(nullptr)
{
}

inline bool FIOSDeviceOutputReaderRunnable::StartSyslogProcess(void)
{
	FString Exe = GetLibImobileDeviceExe("idevicesyslog");
	FString Params = FString::Printf(TEXT(" -u %s -m [UE]"), *DeviceUDID);
	SyslogProcHandle = FPlatformProcess::CreateProc(*Exe, *Params, true, false, false, NULL, 0, NULL, SyslogWritePipe);
	return SyslogProcHandle.IsValid();
}

inline bool FIOSDeviceOutputReaderRunnable::Init(void)
{
	FPlatformProcess::CreatePipe(SyslogReadPipe, SyslogWritePipe);
	return StartSyslogProcess();
}

inline void FIOSDeviceOutputReaderRunnable::Exit(void)
{
	if (SyslogProcHandle.IsValid())
	{
		FPlatformProcess::TerminateProc(SyslogProcHandle);
	}
	FPlatformProcess::ClosePipe(SyslogReadPipe, SyslogWritePipe);
}

inline void FIOSDeviceOutputReaderRunnable::Stop(void)
{
	StopTaskCounter.Increment();
}

inline uint32 FIOSDeviceOutputReaderRunnable::Run(void)
{
	FString SyslogOutput;

	while (StopTaskCounter.GetValue() == 0 && SyslogProcHandle.IsValid())
	{
		if (!FPlatformProcess::IsProcRunning(SyslogProcHandle))
		{
			// When user plugs out USB cable idevicesyslog process stops
			// Keep trying to restore idevicesyslog connection until code that uses this object will not kill us
			Output->Serialize(TEXT("Trying to restore connection to device..."), ELogVerbosity::Log, NAME_None);
			FPlatformProcess::CloseProc(SyslogProcHandle);
			if (StartSyslogProcess())
			{
				FPlatformProcess::Sleep(1.0f);
			}
			else
			{
				Output->Serialize(TEXT("Failed to start idevicesyslog proccess"), ELogVerbosity::Log, NAME_None);
				Stop();
			}
		}
		else
		{
			SyslogOutput.Append(FPlatformProcess::ReadPipe(SyslogReadPipe));

			if (SyslogOutput.Len() > 0)
			{
				TArray<FString> OutputLines;
				SyslogOutput.ParseIntoArray(OutputLines, TEXT("\n"), false);

				if (!SyslogOutput.EndsWith(TEXT("\n")))
				{
					// partial line at the end, do not serialize it until we receive remainder
					SyslogOutput = OutputLines.Last();
					OutputLines.RemoveAt(OutputLines.Num() - 1);
				}
				else
				{
					SyslogOutput.Reset();
				}

				for (int32 i = 0; i < OutputLines.Num(); ++i)
				{
					Output->Serialize(*OutputLines[i], ELogVerbosity::Log, NAME_None);
				}
			}

			FPlatformProcess::Sleep(0.1f);
		}
	}

	return 0;
}

inline bool FIOSTargetDeviceOutput::Init(const FIOSTargetDevice& TargetDevice, FOutputDevice* Output)
{
	check(Output);
	// Output will be produced by background thread
	check(Output->CanBeUsedOnAnyThread());
	DeviceName = TargetDevice.GetName();
	auto* Runnable = new FIOSDeviceOutputReaderRunnable(TargetDevice.GetId().GetDeviceName(), Output);
	DeviceOutputThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(Runnable, TEXT("FIOSDeviceOutputReaderRunnable")));
	return true;
}

