// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSTargetPlatformControls.cpp: Implements the FIOSTargetPlatformControls class.
=============================================================================*/

#include "IOSTargetPlatformControls.h"
#include "Interfaces/IProjectManager.h"
#include "InstalledPlatformInfo.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"
#include "Misc/App.h"
#include "Misc/MonitoredProcess.h"
#include "Logging/MessageLog.h"
#include "AnalyticsEventAttribute.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif
#if WITH_ENGINE
#include "Engine/Texture.h"
#include "Sound/SoundWave.h"
#include "TextureResource.h"
#include "AudioCompressionSettings.h"
#endif

/* FIOSTargetPlatformControls structors
 *****************************************************************************/

FIOSTargetPlatformControls::FIOSTargetPlatformControls(bool bInIsTVOS, bool bInIsVisionOS, bool bIsClientOnly, ITargetPlatformSettings* TargetPlatformSettings)
	// override the ini name up in the base classes, which will go into the FTargetPlatformInfo
	: TNonDesktopTargetPlatformControlsBase(bIsClientOnly, TargetPlatformSettings, nullptr, bInIsTVOS ? TEXT("TVOS") : bInIsVisionOS ? TEXT("VisionOS") : nullptr)
	, bIsTVOS(bInIsTVOS)
	, bIsVisionOS(bInIsVisionOS)
{
	// initialize the connected device detector
	DeviceHelper.OnDeviceConnected().AddRaw(this, &FIOSTargetPlatformControls::HandleDeviceConnected);
	DeviceHelper.OnDeviceDisconnected().AddRaw(this, &FIOSTargetPlatformControls::HandleDeviceDisconnected);
	DeviceHelper.Initialize(bIsTVOS || bIsVisionOS);
}


FIOSTargetPlatformControls::~FIOSTargetPlatformControls()
{
}


/* ITargetPlatform interface
 *****************************************************************************/

void FIOSTargetPlatformControls::EnableDeviceCheck(bool OnOff)
{
	FIOSDeviceHelper::EnableDeviceCheck(OnOff);
}

void FIOSTargetPlatformControls::GetAllDevices( TArray<ITargetDevicePtr>& OutDevices ) const
{
	OutDevices.Reset();

	for (auto Iter = Devices.CreateConstIterator(); Iter; ++Iter)
	{
		OutDevices.Add(Iter.Value());
	}
}


ITargetDevicePtr FIOSTargetPlatformControls::GetDefaultDevice() const
{
	if (Devices.Num() > 0)
	{
		// first device is the default
		auto Iter = Devices.CreateConstIterator();
		if(Iter)
		{
			return Iter.Value();
		}
	}

	return NULL;
}


ITargetDevicePtr FIOSTargetPlatformControls::GetDevice( const FTargetDeviceId& DeviceId )
{
	return Devices.FindRef(DeviceId);
}

static FString OutputMessage;
static void OnOutput(FString Message)
{
    OutputMessage += Message;
    UE_LOGF(LogTemp, Display, "%ls\n", *Message);
}

bool FIOSTargetPlatformControls::IsSdkInstalled(bool bProjectHasCode, FString& OutTutorialPath) const
{
#if PLATFORM_MAC
	OutTutorialPath = FString("Shared/Tutorials/InstallingXCodeTutorial");

	// run xcode-select and get the location of Xcode
	FString CmdExe = TEXT("/usr/bin/xcode-select");
	FString CommandLine = FString::Printf(TEXT("--print-path"));
	TSharedPtr<FMonitoredProcess> IPPProcess = MakeShareable(new FMonitoredProcess(CmdExe, CommandLine, true));
	OutputMessage = TEXT("");
	IPPProcess->OnOutput().BindStatic(&OnOutput);
	IPPProcess->Launch();
	while (IPPProcess->Update())
	{
		FPlatformProcess::Sleep(0.01f);
	}
	int RetCode = IPPProcess->GetReturnCode();
//	UE_LOGF(LogTemp, Display, "%ls", *OutputMessage);

	bool biOSSDKInstalled = IFileManager::Get().DirectoryExists(*OutputMessage);
#else
	OutTutorialPath = FString("/Engine/Tutorial/Mobile/InstallingiTunesTutorial.InstallingiTunesTutorial");

	// On windows we check if itunes is installed - Perhaps someday make this its own check instead of piggy packing on the SDK check which will create a unintuitive error message when it fails

	// The logic here is to assume the correct Apple dll does not exist and then check the various locations it could be in, setting this to true when it is found
	// Code is structured for clarity not performance
	bool biOSSDKInstalled = false;

	HKEY hKey;
	TCHAR dllPath[256];
	unsigned long pathSize = 256;
	
	// Add future version checks here

	// Check for iTunes 12
	if (!biOSSDKInstalled && RegOpenKeyEx(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Wow6432Node\\Apple Inc.\\Apple Mobile Device Support\\Shared"), 0, KEY_READ, &hKey) == ERROR_SUCCESS)
	{
		if (RegQueryValueEx(hKey, TEXT("MobileDeviceDLL"), 0, NULL, (BYTE*)dllPath, &pathSize) == ERROR_SUCCESS
			&& IFileManager::Get().FileSize(*FString(dllPath)) != INDEX_NONE)
		{
			biOSSDKInstalled = true;
		}
		else
		{
			// iTunes >= 12.7 doesn't have a key specifying the 32-bit DLL but it does have a ASMapiInterfaceDLL key and MobileDevice.dll is in usually in the same directory
			if (RegQueryValueEx(hKey, TEXT("ASMapiInterfaceDLL"), 0, NULL, (BYTE*)dllPath, &pathSize) == ERROR_SUCCESS)
			{
				FString MobileDeviceDLLPath = FString(dllPath);
				int32 Index = MobileDeviceDLLPath.Find(TEXT("\\"), ESearchCase::CaseSensitive, ESearchDir::FromEnd);
				if(Index != INDEX_NONE)
				{
					MobileDeviceDLLPath = MobileDeviceDLLPath.Left(Index+1) + TEXT("MobileDevice.dll");
					if (IFileManager::Get().FileSize(*MobileDeviceDLLPath) != INDEX_NONE)
					{
						biOSSDKInstalled = true;
					}
				}
			}
		}
	}

	// Check for iTunes 12, Windows Store version
	if (!biOSSDKInstalled)
	{
		HKEY hPackagesKey;
		FString PackagesKeyName(TEXT("Software\\Classes\\Local Settings\\Software\\Microsoft\\Windows\\CurrentVersion\\AppModel\\PackageRepository\\Packages"));
		if (RegOpenKeyEx(HKEY_LOCAL_MACHINE, *PackagesKeyName, 0, KEY_READ, &hPackagesKey) == ERROR_SUCCESS)
		{
			DWORD NumSubKeys = 0;
			DWORD LongestSubKeyLength;
			if (RegQueryInfoKey(hPackagesKey, NULL, NULL, NULL, &NumSubKeys, &LongestSubKeyLength, NULL, NULL, NULL, NULL, NULL, NULL) == ERROR_SUCCESS)
			{
				static const uint32 MAX_SUBKEY_LENGTH = 512;
				check(LongestSubKeyLength < MAX_SUBKEY_LENGTH);

				for (uint32 i = 0; i < NumSubKeys; i++)
				{
					TCHAR SubKeyName[MAX_SUBKEY_LENGTH];
					DWORD SubKeyLength = MAX_SUBKEY_LENGTH;
					if (RegEnumKeyEx(hPackagesKey, i, SubKeyName, &SubKeyLength, NULL, NULL, NULL, NULL) == ERROR_SUCCESS)
					{
						check(SubKeyLength < MAX_SUBKEY_LENGTH);

						if (FString(SubKeyName).Contains(TEXT("AppleInc.iTunes")) && (FString(SubKeyName).Contains(TEXT("_x64")) || FString(SubKeyName).Contains(TEXT("_x86"))))
						{
							HKEY iTunesKey;
							TCHAR FullPackageSubKeyName[MAX_SUBKEY_LENGTH + MAX_SUBKEY_LENGTH + 1];
							_stprintf_s(FullPackageSubKeyName, MAX_SUBKEY_LENGTH + MAX_SUBKEY_LENGTH + 1, TEXT("%s\\%s"), *PackagesKeyName, SubKeyName);
							if (RegOpenKeyEx(HKEY_LOCAL_MACHINE, FullPackageSubKeyName, 0, KEY_READ, &iTunesKey) == ERROR_SUCCESS)
							{
								if (RegQueryValueEx(iTunesKey, TEXT("Path"), NULL, NULL, (BYTE*)dllPath, &pathSize) == ERROR_SUCCESS)
								{
									TCHAR dllFullPath[MAX_SUBKEY_LENGTH + MAX_SUBKEY_LENGTH + 32];
									_stprintf_s(dllFullPath, MAX_SUBKEY_LENGTH + MAX_SUBKEY_LENGTH + 32, TEXT("%s\\AMDS32\\MobileDevice.dll"), dllPath);

									if (IFileManager::Get().FileSize(*FString(dllFullPath)) != INDEX_NONE)
									{
										biOSSDKInstalled = true;
										break;
									}
								}
							}
						}
					}
				}
			}
		}
	}

	// Check for iTunes 11
	if(!biOSSDKInstalled
		&& RegOpenKeyEx(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Wow6432Node\\Apple Inc.\\Apple Mobile Device Support\\Shared"), 0, KEY_READ, &hKey) == ERROR_SUCCESS
		&& RegQueryValueEx(hKey, TEXT("iTunesMobileDeviceDLL"), 0, NULL, (BYTE*)dllPath, &pathSize) == ERROR_SUCCESS
		&&  IFileManager::Get().FileSize(*FString(dllPath)) != INDEX_NONE)
			{
		biOSSDKInstalled = true;
	}

#endif
	return biOSSDKInstalled;
}

int32 FIOSTargetPlatformControls::CheckRequirements(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FString& OutTutorialPath, FString& OutDocumentationPath, FText& CustomizedLogMessage) const
{
	OutDocumentationPath = TEXT("Platforms/iOS/QuickStart/6");

	int32 bReadyToBuild = ETargetPlatformReadyStatus::Ready; // @todo How do we check that the iOS SDK is installed when building from Windows? Is that even possible?
	if (!IsSdkInstalled(bProjectHasCode, OutTutorialPath))
	{
		bReadyToBuild |= ETargetPlatformReadyStatus::SDKNotFound;
	}
#if PLATFORM_MAC
	OutTutorialPath = FString("/Engine/Tutorial/Installation/InstallingXCodeTutorial.InstallingXCodeTutorial");
    // shell to certtool
#else
	if (!FInstalledPlatformInfo::Get().IsValidPlatform(GetPlatformInfo().UBTPlatformString, EProjectType::Code))
	{
		if (bProjectHasCode)
		{
			OutTutorialPath = FString("/Engine/Tutorial/Mobile/iOSonPCRestrictions.iOSonPCRestrictions");
			bReadyToBuild |= ETargetPlatformReadyStatus::CodeUnsupported;
		}

		FText Reason;
		if (RequiresTempTarget(bProjectHasCode, Configuration, bRequiresAssetNativization, Reason))
		{
			OutTutorialPath = FString("/Engine/Tutorial/Mobile/iOSonPCValidPlugins.iOSonPCValidPlugins");
			bReadyToBuild |= ETargetPlatformReadyStatus::PluginsUnsupported;
		}
	}
#endif

	// shell to IPP and get the status of the provision and cert

	bool bForDistribtion = false;
	GConfig->GetBool(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("ForDistribution"), bForDistribtion, GGameIni);

	FString BundleIdentifier;
	GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("BundleIdentifier"), BundleIdentifier, GEngineIni);
	BundleIdentifier = BundleIdentifier.Replace(TEXT("[PROJECT_NAME]"), FApp::GetProjectName());
	BundleIdentifier = BundleIdentifier.Replace(TEXT("_"), TEXT(""));

	bool bAutomaticSigning = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bAutomaticSigning"), bAutomaticSigning, GEngineIni);

	FString TeamID;
	GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("IOSTeamID"), TeamID, GEngineIni);

	FString ProjectPath = FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());
#if PLATFORM_MAC
    FString CmdExe = TEXT("/bin/sh");
    FString ScriptPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Build/BatchFiles/RunDotnet.sh"));
    FString IPPPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/DotNET/IOS/IPhonePackagerCLI.dll"));
	FString CommandLine = FString::Printf(TEXT("\"%s\" \"%s\" Validate Engine -project \"%s\" -bundlename \"%s\" -teamID \"%s\" %s %s"), *ScriptPath, *IPPPath, *ProjectPath, *(BundleIdentifier), *(TeamID), (bForDistribtion ? TEXT("-distribution") : TEXT("")), bAutomaticSigning ? TEXT("-autosigning") : TEXT(""));
#else
	FString CmdExe = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/DotNET/IOS/IPhonePackager.exe"));
	FString CommandLine = FString::Printf(TEXT("Validate Engine -project \"%s\" -bundlename \"%s\" %s"), *ProjectPath, *(BundleIdentifier), (bForDistribtion ? TEXT("-distribution") : TEXT("")) );
	FString RemoteServerName;
	FString SecondaryRemoteServerName;
	FString RSyncUsername;
	FString SecondaryRSyncUsername;
	GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("RemoteServerName"), RemoteServerName, GEngineIni);
	GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("RSyncUsername"), RSyncUsername, GEngineIni);
	GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("SecondaryRemoteServerName"), SecondaryRemoteServerName, GEngineIni);
	GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("SecondaryRSyncUsername"), SecondaryRSyncUsername, GEngineIni);
	if (RemoteServerName.Len() == 0 || RSyncUsername.Len() == 0)
	{
		bReadyToBuild |= ETargetPlatformReadyStatus::RemoveServerNameEmpty;
	}

#endif
	if (bIsTVOS)
	{
		CommandLine += " -tvos";
	}
	TSharedPtr<FMonitoredProcess> IPPProcess = MakeShareable(new FMonitoredProcess(CmdExe, CommandLine, true));
	OutputMessage = TEXT("");
	IPPProcess->OnOutput().BindStatic(&OnOutput);
	IPPProcess->Launch();
	while(IPPProcess->Update())
	{
		FPlatformProcess::Sleep(0.01f);
	}
	int RetCode = IPPProcess->GetReturnCode();
//	UE_LOGF(LogTemp, Display, "%ls", *OutputMessage);
	if (RetCode == 14)
	{
		OutTutorialPath = FString("/Engine/Tutorial/Mobile/CreatingInfoPlist.CreatingInfoPlist");
		bReadyToBuild |= ETargetPlatformReadyStatus::ManifestNotFound;
	}
	else if (RetCode == 13)
	{
		OutTutorialPath = FString("/Engine/Tutorial/Mobile/CreatingSigningCertAndProvisionTutorial.CreatingSigningCertAndProvisionTutorial");
		bReadyToBuild |= ETargetPlatformReadyStatus::SigningKeyNotFound;
		bReadyToBuild |= ETargetPlatformReadyStatus::ProvisionNotFound;
	}
	else if (RetCode == 12)
	{
		OutTutorialPath = FString("/Engine/Tutorial/Mobile/CreatingSigningCertAndProvisionTutorial.CreatingSigningCertAndProvisionTutorial");
		bReadyToBuild |= ETargetPlatformReadyStatus::SigningKeyNotFound;
	}
	else if (RetCode == 11)
	{
		OutTutorialPath = FString("/Engine/Tutorial/Mobile/CreatingSigningCertAndProvisionTutorial.CreatingSigningCertAndProvisionTutorial");
		bReadyToBuild |= ETargetPlatformReadyStatus::ProvisionNotFound;
	}

	{
		TArray<FString> FoundIconFiles;
		FString Wildcard = FPaths::Combine(FPaths::ProjectDir(), TEXT("Build"), TEXT("IOS"), TEXT("Resources"), TEXT("Graphics"), TEXT("Icon*.png"));
		IFileManager::Get().FindFiles(FoundIconFiles, *Wildcard, true, false);
		if (FoundIconFiles.Num() > 0)
		{
			bReadyToBuild |= ETargetPlatformReadyStatus::CodeBuildRequired;
		}
	}

	return bReadyToBuild;
}



/* FIOSTargetPlatformControls callbacks
 *****************************************************************************/

void FIOSTargetPlatformControls::HandlePongMessage( const FIOSLaunchDaemonPong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	FTargetDeviceId DeviceId;
	FTargetDeviceId::Parse(Message.DeviceID, DeviceId);

	FIOSTargetDevicePtr& Device = Devices.FindOrAdd(DeviceId);

	if (!Device.IsValid())
	{
		Device = MakeShareable(new FIOSTargetDevice(*this));

		Device->SetFeature(ETargetDeviceFeatures::Reboot, Message.bCanReboot);
		Device->SetFeature(ETargetDeviceFeatures::PowerOn, Message.bCanPowerOn);
		Device->SetFeature(ETargetDeviceFeatures::PowerOff, Message.bCanPowerOff);
		Device->SetDeviceId(DeviceId);
		Device->SetDeviceName(Message.DeviceName);
		Device->SetDeviceType(Message.DeviceType);
		Device->SetModelId(Message.DeviceModelId);
		Device->SetOSVersion(Message.DeviceOSVersion);
		Device->SetDeviceConnectionType(Message.DeviceConnectionType);
		Device->SetDeviceEndpoint(Context->GetSender());
		Device->SetIsSimulated(Message.DeviceID.Contains(TEXT("Simulator")));

		ITargetPlatformControls::OnDeviceDiscovered().Broadcast(Device.ToSharedRef());
	}

	Device->LastPinged = FDateTime::UtcNow();
}

void FIOSTargetPlatformControls::HandleDeviceConnected(const FIOSLaunchDaemonPong& Message)
{
	FTargetDeviceId DeviceId;
	FTargetDeviceId::Parse(Message.DeviceID, DeviceId);
	
	FIOSTargetDevicePtr& Device = Devices.FindOrAdd(DeviceId);
	
	if (!Device.IsValid())
	{
		bool bIsTVOSDevice = Message.DeviceType.Contains(TEXT("AppleTV"));
		bool bIsVisionOSDevice = Message.DeviceType.Contains(TEXT("RealityDevice"));
		bool bIsIOSDevice = !bIsTVOSDevice && !bIsVisionOSDevice;
		
		bool bIsIOS = !bIsTVOS && !bIsVisionOS;

		if ((bIsIOS && bIsIOSDevice) || (bIsTVOS && bIsTVOSDevice) || (bIsVisionOS && bIsVisionOSDevice))
		{
			Device = MakeShareable(new FIOSTargetDevice(*this));

			Device->SetFeature(ETargetDeviceFeatures::Reboot, Message.bCanReboot);
			Device->SetFeature(ETargetDeviceFeatures::PowerOn, Message.bCanPowerOn);
			Device->SetFeature(ETargetDeviceFeatures::PowerOff, Message.bCanPowerOff);
			Device->SetDeviceId(DeviceId);
			Device->SetDeviceName(Message.DeviceName);
			Device->SetAuthorized(Message.bIsAuthorized);
			Device->SetDeviceType(Message.DeviceType);
			Device->SetModelId(Message.DeviceModelId);
			Device->SetOSVersion(Message.DeviceOSVersion);
			Device->SetDeviceConnectionType(Message.DeviceConnectionType);
			Device->SetIsSimulated(Message.DeviceID.Contains(TEXT("Simulator")));

			OnDeviceDiscovered().Broadcast(Device.ToSharedRef());
		}
		else
		{
			return;
		}
	}
	
	// Add a very long time period to prevent the devices from getting disconnected due to a lack of pong messages
	Device->LastPinged = FDateTime::UtcNow() + FTimespan::FromDays(100.0);
}


void FIOSTargetPlatformControls::HandleDeviceDisconnected(const FIOSLaunchDaemonPong& Message)
{
	FTargetDeviceId DeviceId;
	FTargetDeviceId::Parse(Message.DeviceID, DeviceId);
	
	FIOSTargetDevicePtr& Device = Devices.FindOrAdd(DeviceId);
	
	if (Device.IsValid())
	{
		ITargetPlatformControls::OnDeviceLost().Broadcast(Device.ToSharedRef());
		Devices.Remove(DeviceId);
	}
}


/* ITargetPlatform interface
 *****************************************************************************/
static bool SupportsMetal()
{
	// default to NOT supporting metal
	bool bSupportsMetal = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetal"), bSupportsMetal, GEngineIni);
	return bSupportsMetal;
}

static bool SupportsMetalMobileSM5()
{
	bool bSupportsMetalMobileSM5 = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM5"), bSupportsMetalMobileSM5, GEngineIni);
	return bSupportsMetalMobileSM5;
}

static bool SupportsMetalMobileSM6()
{
	bool bSupportsMetalMobileSM6 = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM6"), bSupportsMetalMobileSM6, GEngineIni);
	return bSupportsMetalMobileSM6;
}

void FIOSTargetPlatformControls::GetPlatformSpecificProjectAnalytics( TArray<FAnalyticsEventAttribute>& AnalyticsParamArray ) const
{
	TNonDesktopTargetPlatformControlsBase<FIOSPlatformProperties>::GetPlatformSpecificProjectAnalytics( AnalyticsParamArray );

	AppendAnalyticsEventAttributeArray(AnalyticsParamArray,
		TEXT("SupportsMetalMobileSM5"), SupportsMetalMobileSM5(),
		TEXT("SupportsMetalMobileSM6"), SupportsMetalMobileSM6()
	);
}

#if WITH_ENGINE

static const FName NameASTC_RGB_HDR(TEXT("ASTC_RGB_HDR"));
static const FName NameASTC_RGB_LDR(TEXT("ASTC_RGB"));
static const FName NameBC5(TEXT("BC5"));
static const FName NameBC4(TEXT("BC4"));
static const FName NameASTC_NormalLA(TEXT("ASTC_NormalLA"));

// we remap some of the defaults
static const FName FormatRemap[] =
{
	// original				ASTC
	FName(TEXT("AutoDXT")),	FName(TEXT("ASTC_RGBAuto")),
	FName(TEXT("DXT1")),	FName(TEXT("ASTC_RGB")),
	FName(TEXT("DXT5")),	FName(TEXT("ASTC_RGBA")),
	FName(TEXT("DXT5n")),	FName(TEXT("ASTC_NormalAG")),
	NameBC5,				FName(TEXT("ASTC_NormalRG")),
	FName(TEXT("BC4")),		FName(TEXT("ETC2_R11")),
	FName(TEXT("BC6H")),	NameASTC_RGB_HDR,
	FName(TEXT("BC7")),		FName(TEXT("ASTC_RGBA_HQ"))
};
static const FName NameG8(TEXT("G8"));
static const FName NameRGBA16F(TEXT("RGBA16F"));
static const FName NameR16F(TEXT("R16F"));

void FIOSTargetPlatformControls::GetTextureFormats( const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const
{
	check(Texture);

	const int32 NumLayers = Texture->Source.GetNumLayers();
	
	TArray<FName>& TextureFormatNames = OutFormats.AddDefaulted_GetRef();
	TextureFormatNames.Reserve(NumLayers);

	// optionaly compress landscape weightmaps for a mobile rendering
	// @todo Oodle: this should not be here
	//	should be in GetDefaultTextureFormatNamePerLayer
	//	so that 4x4 checks can be applied correctly, etc.
	bool bCompressLandscapeWeightMaps = false;
	GetTargetPlatformSettings()->GetConfigSystem()->GetBool(TEXT("/Script/Engine.RendererSettings"), TEXT("r.Mobile.CompressLandscapeWeightMaps"), bCompressLandscapeWeightMaps, GEngineIni);

	if (Texture->LODGroup == TEXTUREGROUP_Terrain_Weightmap && bCompressLandscapeWeightMaps)
	{
		TextureFormatNames.Init(FName(TEXT("AutoDXT")), NumLayers);
	}

	// if we didn't assign anything specially, then use the defaults
	if (TextureFormatNames.Num() == 0)
	{
        int32 BlockSize = 1;
		// Compressed volume textures require MTLGPUFamilyApple3 or later which is our minimal supported
		bool bSupportCompressedVolumeTexture = true;
		bool bSupportFilteredFloat32Textures = false;
		GetDefaultTextureFormatNamePerLayer(TextureFormatNames, this->GetTargetPlatformSettings(), this, Texture, bSupportCompressedVolumeTexture, BlockSize, bSupportFilteredFloat32Textures);
	}

	// L+A mode for normal map compression
	const bool bSupportsNormalLA = GetTargetPlatformSettings()->SupportsFeature(ETargetPlatformFeatures::NormalmapLAEncodingMode);

	// include the formats we want
	for (FName& TextureFormatName : TextureFormatNames)
	{
		if (bSupportsNormalLA && TextureFormatName == NameBC5)
		{
			TextureFormatName = NameASTC_NormalLA;
			continue;
		}
		
		// Metal does not support ETC2_11 on 3D textures
		if (TextureFormatName == NameBC4 && Texture->GetTextureClass() == ETextureClass::Volume)
		{
			TextureFormatName = NameR16F;
			continue;
		}

		for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(FormatRemap); RemapIndex += 2)
		{
			if (TextureFormatName == FormatRemap[RemapIndex])
			{
				TextureFormatName = FormatRemap[RemapIndex + 1];
				break;
			}
		}
	}
	
	bool bSupportASTCHDR = GetTargetPlatformSettings()->UsesASTCHDR();

	if ( ! bSupportASTCHDR )
	{
		for (FName& TextureFormatName : TextureFormatNames)
		{
			if ( TextureFormatName == NameASTC_RGB_HDR )
			{
				TextureFormatName = GetTargetPlatformSettings()->GetFallbackASTCHDR();
			}
		}
	}

	for (FName& TextureFormatName : OutFormats.Last())
	{
		if (Texture->GetTextureClass() == ETextureClass::Cube) 
		{
			FTextureFormatSettings FormatSettings;
			Texture->GetDefaultFormatSettings(FormatSettings);
			// TC_EncodedReflectionCapture is no longer used and could be deleted
			if (FormatSettings.CompressionSettings == TC_EncodedReflectionCapture && !FormatSettings.CompressionNone)
			{
				TextureFormatName = FName(TEXT("ETC2_RGBA"));
			}
		}
	}
}

FName FIOSTargetPlatformControls::FinalizeVirtualTextureLayerFormat(FName Format) const
{
#if WITH_EDITOR

	// VirtualTexture Format was already run through the ordinary texture remaps to change AutoDXT to ASTC or ETC
	// this then runs again
	// currently it forces all ASTC to ETC
	// this is needed because the runtime virtual texture encoder only supports ETC
	
	// code dupe with AndroidTargetPlatform

	const static FName NameETC2_RGB(TEXT("ETC2_RGB"));
	const static FName NameETC2_RGBA(TEXT("ETC2_RGBA"));
	const static FName NameAutoETC2(TEXT("AutoETC2"));
	const static FName NameETC2_RG11(TEXT("ETC2_RG11"));

	// Remap non-ETC variants to ETC
	const static FName ETCRemap[][2] =
	{
		{ { FName(TEXT("ASTC_RGB")) },			{ NameETC2_RGB } },
		{ { FName(TEXT("ASTC_RGBA")) },			{ NameETC2_RGBA } },
		{ { FName(TEXT("ASTC_RGBAuto")) },		{ NameAutoETC2 } },
		{ { FName(TEXT("ASTC_RGBA_HQ")) },		{ NameETC2_RGBA } },
//		{ { FName(TEXT("ASTC_RGB_HDR")) },		{ NameRGBA16F } }, // ?
		{ { FName(TEXT("ASTC_NormalAG")) },		{ NameETC2_RGB } },
		{ { FName(TEXT("ASTC_NormalRG")) },		{ NameETC2_RG11 } },
		{ { FName(TEXT("ASTC_NormalLA")) },	    { NameETC2_RG11 } },
	};

	for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(ETCRemap); RemapIndex++)
	{
		if (ETCRemap[RemapIndex][0] == Format)
		{
			return ETCRemap[RemapIndex][1];
		}
	}
#endif
	return Format;
}
#endif // WITH_ENGINE

