// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSTargetPlatformControls.h: Declares the FIOSTargetPlatformControls class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Common/TargetPlatformControlsBase.h"
#include "IOS/IOSPlatformProperties.h"
#include "Containers/Ticker.h"
#include "IOSMessageProtocol.h"
#include "IMessageContext.h"
#include "IOSTargetDevice.h"
#include "IOSDeviceHelper.h"
#include "Misc/ConfigCacheIni.h"


#if WITH_ENGINE
#include "AudioCompressionSettings.h"
#endif // WITH_ENGINE

/**
 * FIOSTargetPlatformControls, abstraction for cooking iOS platforms
 */
class FIOSTargetPlatformControls : public TNonDesktopTargetPlatformControlsBase<FIOSPlatformProperties>
{
public:

	/**
	 * Default constructor.
	 */
	IOSTARGETPLATFORMCONTROLS_API FIOSTargetPlatformControls(bool bInISTVOS, bool bInIsVisionOS, bool bInIsClientOnly, ITargetPlatformSettings* TargetPlatformSettings);

	/**
	 * Destructor.
	 */
	~FIOSTargetPlatformControls();

public:

	virtual void EnableDeviceCheck(bool OnOff) override;

	virtual void GetAllDevices( TArray<ITargetDevicePtr>& OutDevices ) const override;

	virtual bool GenerateStreamingInstallManifest(const TMultiMap<FString, int32>& PakchunkMap, const TSet<int32>& PakchunkIndicesInUse) const override
	{
		return true;
	}

	virtual ITargetDevicePtr GetDefaultDevice( ) const override;

	virtual ITargetDevicePtr GetDevice( const FTargetDeviceId& DeviceId ) override;

	virtual bool IsSdkInstalled(bool bProjectHasCode, FString& OutTutorialPath) const override;
	virtual int32 CheckRequirements(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FString& OutTutorialPath, FString& OutDocumentationPath, FText& CustomizedLogMessage) const override;

	virtual void GetPlatformSpecificProjectAnalytics( TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray ) const override;

#if WITH_ENGINE

	virtual void GetTextureFormats( const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const override;

	virtual FName FinalizeVirtualTextureLayerFormat(FName Format) const override;

#endif // WITH_ENGINE

	virtual void GetBuildProjectSettingKeys(FString& OutSection, TArray<FString>& InBoolKeys, TArray<FString>& InIntKeys, TArray<FString>& InStringKeys) const override
	{
		OutSection = TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings");
		InBoolKeys.Add(TEXT("bGeneratedSYMFile"));
		InBoolKeys.Add(TEXT("bGeneratedSYMBundle"));
		InBoolKeys.Add(TEXT("bGenerateXCArchive"));
		if (bIsTVOS)
		{
			InStringKeys.Add(TEXT("MinimumTVOSVersion"));
		}
		else
		{
			InStringKeys.Add(TEXT("MinimumiOSVersion"));
		}
	}

	//~ Begin ITargetPlatform Interface

private:

	// Handles received pong messages from the LauncherDaemon.
	void HandlePongMessage( const FIOSLaunchDaemonPong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

    void HandleDeviceConnected( const FIOSLaunchDaemonPong& Message );
    void HandleDeviceDisconnected( const FIOSLaunchDaemonPong& Message );

private:
	
	// true if this is targeting TVOS vs IOS
	bool bIsTVOS;
	bool bIsVisionOS;

	// Contains all discovered IOSTargetDevices over the network.
	TMap<FTargetDeviceId, FIOSTargetDevicePtr> Devices;

	// Holds the message endpoint used for communicating with the LaunchDaemon.
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

    // holds usb device helper
	FIOSDeviceHelper DeviceHelper;

};
