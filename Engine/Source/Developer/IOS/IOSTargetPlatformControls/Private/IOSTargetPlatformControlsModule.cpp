// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSTargetPlatformControls.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Common/TargetPlatformSettingsBase.h"
#include "Modules/ModuleManager.h"


/**
 * Module for iOS as a target platform controls
 */
class FIOSTargetPlatformControlsModule : public ITargetPlatformControlsModule
{
public:
	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName) override
	{
		if (FIOSTargetPlatformControls::IsUsable())
		{
			ITargetPlatformSettingsModule* ModuleSettings = FModuleManager::GetModulePtr<ITargetPlatformSettingsModule>(PlatformSettingsModuleName);
			if (ModuleSettings != nullptr)
			{
				TArray<ITargetPlatformSettings*> TargetPlatformSettings = ModuleSettings->GetTargetPlatformSettings();
				check(TargetPlatformSettings.Num() == 1);

				TargetPlatforms.Add(new FIOSTargetPlatformControls(false, false, false, TargetPlatformSettings[0]));
				TargetPlatforms.Add(new FIOSTargetPlatformControls(false, false, true, TargetPlatformSettings[0]));
			}
		}
	}
};


IMPLEMENT_MODULE(FIOSTargetPlatformControlsModule, IOSTargetPlatformControls);
