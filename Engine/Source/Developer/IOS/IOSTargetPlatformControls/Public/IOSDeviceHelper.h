// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum EDeviceConnectionInterface
{
	NoValue = 0,
	USB = 1,
	Network = 2,
	Simulator = 3,
	Max = 4
};

struct FDeviceNotificationCallbackInformation
{
	FString DeviceID;
	FString DeviceName;
	FString DeviceUDID;
	FString ProductType;
	FString DeviceOSVersion;
	EDeviceConnectionInterface DeviceInterface;
	uint32 msgType;
	bool IsAuthorized;
};

struct FLibIMobileDevice
{
	FString DeviceID;
	FString DeviceName;
	FString DeviceUDID;
	FString DeviceType;
	FString DeviceOSVersion;
	EDeviceConnectionInterface DeviceInterface;
	bool IsAuthorized;
	bool IsDealtWith;
};

/**
 * Type definition for shared pointers to instances of FIOSDeviceHelper
 */
typedef TSharedPtr<class FIOSDeviceHelper, ESPMode::ThreadSafe> FIOSDeviceHelperPtr;

/**
 * Type definition for shared references to instances of FIOSDeviceHelper
 */
typedef TSharedRef<class FIOSDeviceHelper, ESPMode::ThreadSafe> FIOSDeviceHelperRef;

/**
 * Delegate type for devices being connected or disconnected from the machine
 *
 * The first parameter is newly added or removed device
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDeviceConnectEvent, const struct FIOSLaunchDaemonPong&);

/**
 * Implemented the iOS device helper class
 */
class FIOSDeviceHelper
{
public:
	static  void Initialize(bool bIsTVOS);
	static IOSTARGETPLATFORMCONTROLS_API TArray<FLibIMobileDevice> GetLibIMobileDevices();
	/**
	 * Returns a delegate that is executed when a device is connected
	 *
	 * @return the delegate
	 */
	static FOnDeviceConnectEvent& OnDeviceConnected()
	{
		static FOnDeviceConnectEvent DeviceConnectedDelegate;
		return DeviceConnectedDelegate;
	}
	
	/**
	 * Returns a delegate the is executed when a device is disconnected
	 *
	 * @return the delegate
	 */
	static FOnDeviceConnectEvent& OnDeviceDisconnected()
	{
		static FOnDeviceConnectEvent DeviceDisconnectedDelegate;
		return DeviceDisconnectedDelegate;
	}
	
	/**
	 * Installs an ipa on to the device
	 */
	static bool InstallIPAOnDevice(const class FTargetDeviceId& DeviceId, const FString& IPAPath);
	
	/**
	 * Suspends/Enables the device connect/disconnect thread
	 */
	static void EnableDeviceCheck(bool OnOff);
	
	static const FString LocalMacDummyUUID;

private:
	static void DeviceCallback(void*);
	static void DoDeviceConnect(void*);
	static void DoDeviceDisconnect(void*);
	static bool MessageTickDelegate(float);
};
