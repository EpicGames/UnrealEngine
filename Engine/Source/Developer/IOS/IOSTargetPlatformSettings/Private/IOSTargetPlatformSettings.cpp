// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSTargetPlatformSettings.cpp: Implements the FIOSTargetPlatformSettings class.
=============================================================================*/

#include "IOSTargetPlatformSettings.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"
#include "Misc/App.h"

#if WITH_ENGINE
#include "Engine/Texture.h"
#include "TextureResource.h"
#endif

static bool SupportsMetal()
{
	// default to NOT supporting metal
	bool bSupportsMetal = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetal"), bSupportsMetal, GEngineIni);
	return bSupportsMetal;
}

// Metal Mobile SM5 represents the SM5 variant of metal for iOS/tvOS
static bool SupportsMetalMobileSM5()
{
	bool bSupportsMetalMobileSM5 = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM5"), bSupportsMetalMobileSM5, GEngineIni);
	return bSupportsMetalMobileSM5;
}

// Metal Mobile SM6 represents the SM6 variant of metal for iOS
static bool SupportsMetalMobileSM6()
{
	bool bSupportsMetalMobileSM6 = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM6"), bSupportsMetalMobileSM6, GEngineIni);
	return bSupportsMetalMobileSM6;
}

/* FIOSTargetPlatformSettings structors
 *****************************************************************************/

FIOSTargetPlatformSettings::FIOSTargetPlatformSettings(bool bInIsTVOS, bool bInIsVisionOS)
	// override the ini name up in the base classes, which will go into the FTargetPlatformInfo
	: TTargetPlatformSettingsBase(nullptr, bInIsVisionOS ? TEXT("VisionOS") : nullptr)
	, bIsTVOS(bInIsTVOS)
	, bIsVisionOS(bInIsVisionOS)
{
#if WITH_ENGINE
	TextureLODSettings = nullptr; // TextureLODSettings are registered by the device profile.
	StaticMeshLODSettings.Initialize(this);
#endif // #if WITH_ENGINE
}

// Using GetConfigSystem instead of GetPlatformValueVariable here because RenderUtils calls this too early.
bool FIOSTargetPlatformSettings::GetIsDistanceField() const
{
	static TOptional<bool> bDistanceField;
	if (!bDistanceField.IsSet())
	{
		bDistanceField = false;
#if WITH_ENGINE
		static IConsoleVariable* CvarDistanceFields = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DistanceFields"));
		if (CvarDistanceFields != nullptr)
		{
			bDistanceField = CvarDistanceFields->GetInt() != 0;
		}

		GetConfigSystem()->GetBool(TEXT("/Script/Engine.RendererSettings"), TEXT("r.DistanceFields"), bDistanceField.GetValue(), GEngineIni);
#endif
		
		// If we support Mobile SM6, force distance fields on
		if (SupportsMetalMobileSM6())
		{
			bDistanceField.GetValue() = true;
		}
	}
	
	return bDistanceField.GetValue();
}

int32 FIOSTargetPlatformSettings::GetMobileShadingPath() const
{
	static TOptional<int32> MobileShadingPath;
	if (!MobileShadingPath.IsSet())
	{
		MobileShadingPath = 0;
#if WITH_ENGINE
		static IConsoleVariable* CvarShadingPath = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.ShadingPath"));
		if (CvarShadingPath != nullptr)
		{
			MobileShadingPath = CvarShadingPath->GetPlatformValueVariable(*IniPlatformName())->GetInt();
		}
#endif
	}
	return MobileShadingPath.GetValue();
}

bool FIOSTargetPlatformSettings::GetIsMobileForwardEnableClusteredReflections() const
{
	static TOptional<bool> bMobileForwardEnableClusteredReflections;
	if (!bMobileForwardEnableClusteredReflections.IsSet())
	{
		bMobileForwardEnableClusteredReflections = false;
#if WITH_ENGINE
		static IConsoleVariable* CvarFwdEnableClusteredReflections = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.Forward.EnableClusteredReflections"));
		if (CvarFwdEnableClusteredReflections != nullptr)
		{
			bMobileForwardEnableClusteredReflections = CvarFwdEnableClusteredReflections->GetPlatformValueVariable(*IniPlatformName())->GetInt() != 0;
		}
#endif
	}
	return bMobileForwardEnableClusteredReflections.GetValue();
}

bool FIOSTargetPlatformSettings::UsesDistanceFields() const
{
	return GetIsDistanceField();
}

FIOSTargetPlatformSettings::~FIOSTargetPlatformSettings()
{
}

/* ITargetPlatform interface
 *****************************************************************************/

bool FIOSTargetPlatformSettings::SupportsFeature( ETargetPlatformFeatures Feature ) const
{
	switch (Feature)
	{
		case ETargetPlatformFeatures::Packaging:
		case ETargetPlatformFeatures::DeviceOutputLog:
			return true;

		case ETargetPlatformFeatures::MobileRendering:
		case ETargetPlatformFeatures::LowQualityLightmaps:
			return SupportsMetal();
			
		case ETargetPlatformFeatures::DeferredRendering:
		case ETargetPlatformFeatures::HighQualityLightmaps:
			return SupportsMetalMobileSM5() || SupportsMetalMobileSM6();

		case ETargetPlatformFeatures::DistanceFieldAO:
			return UsesDistanceFields();

		case ETargetPlatformFeatures::NormalmapLAEncodingMode:
			return true;

		case ETargetPlatformFeatures::SupportsMultipleConnectionTypes:
			return true;

		default:
			break;
	}
	
	return TTargetPlatformSettingsBase<FIOSPlatformProperties>::SupportsFeature(Feature);
}

void FIOSTargetPlatformSettings::GetAllPossibleShaderFormats( TArray<FName>& OutFormats ) const
{
	static FName NAME_SF_METAL_ES3_1_IOS(TEXT("SF_METAL_ES3_1_IOS"));
	static FName NAME_SF_METAL_SIM(TEXT("SF_METAL_SIM"));
	static FName NAME_SF_METAL_SM5_IOS(TEXT("SF_METAL_SM5_IOS"));
	static FName NAME_SF_METAL_SM6_IOS(TEXT("SF_METAL_SM6_IOS"));
	static FName NAME_SF_METAL_ES3_1_TVOS(TEXT("SF_METAL_ES3_1_TVOS"));
	static FName NAME_SF_METAL_SM5_TVOS(TEXT("SF_METAL_SM5_TVOS"));

	if (bIsTVOS)
	{
		if (SupportsMetalMobileSM5())
		{
			OutFormats.AddUnique(NAME_SF_METAL_SM5_TVOS);
		}

		// because we are currently using IOS settings, we will always use metal, even if Metal isn't listed as being supported
		// however, if MetalMobileSM5 is specified and Metal is set to false, then we will just use MetalMobileSM5
		if (SupportsMetal() || !SupportsMetalMobileSM5())
		{
			OutFormats.AddUnique(NAME_SF_METAL_ES3_1_TVOS);
		}
	}
	else
	{
		if (SupportsMetal())
		{
			OutFormats.AddUnique(NAME_SF_METAL_ES3_1_IOS);
		}

		if (SupportsMetalMobileSM6())
		{
			OutFormats.AddUnique(NAME_SF_METAL_SM6_IOS);
		}

		if (SupportsMetalMobileSM5())
		{
			OutFormats.AddUnique(NAME_SF_METAL_SM5_IOS);
		}
	}
}

bool FIOSTargetPlatformSettings::UsesRayTracing() const
{
	bool bEnableRayTracing = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bEnableRayTracing"), bEnableRayTracing, GEngineIni);
 
	return bEnableRayTracing;
}

ERayTracingRuntimeMode FIOSTargetPlatformSettings::GetRayTracingMode() const
{
	if (!TTargetPlatformSettingsBase<FIOSPlatformProperties>::UsesRayTracing())
	{
		return ERayTracingRuntimeMode::Disabled;
	}

	return TTargetPlatformSettingsBase<FIOSPlatformProperties>::ParseRuntimeRayTracingMode(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"));
}

void FIOSTargetPlatformSettings::GetAllTargetedShaderFormats( TArray<FName>& OutFormats ) const
{
	GetAllPossibleShaderFormats(OutFormats);
}

#if WITH_ENGINE

void FIOSTargetPlatformSettings::GetReflectionCaptureFormats( TArray<FName>& OutFormats ) const
{
	const bool bMobileDeferredShading = (GetMobileShadingPath() == 1);

	if (SupportsMetalMobileSM5() || SupportsMetalMobileSM6() || bMobileDeferredShading || GetIsMobileForwardEnableClusteredReflections())
	{
		OutFormats.Add(FName(TEXT("FullHDR")));
	}

	OutFormats.Add(FName(TEXT("EncodedHDR")));
}

const UTextureLODSettings& FIOSTargetPlatformSettings::GetTextureLODSettings() const
{
	return *TextureLODSettings;
}

#endif // WITH_ENGINE

