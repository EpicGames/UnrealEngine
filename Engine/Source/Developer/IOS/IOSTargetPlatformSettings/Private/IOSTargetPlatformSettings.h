// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSTargetPlatformSettings.h: Declares the FIOSTargetPlatformSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Common/TargetPlatformSettingsBase.h"
#include "IOS/IOSPlatformProperties.h"
#include "Misc/ConfigCacheIni.h"

#if WITH_ENGINE
#include "StaticMeshResources.h"
#endif // WITH_ENGINE

/**
 * FIOSTargetPlatformSettings
 */
class FIOSTargetPlatformSettings : public TTargetPlatformSettingsBase<FIOSPlatformProperties>
{
public:

	/**
	 * Default constructor.
	 */
	IOSTARGETPLATFORMSETTINGS_API FIOSTargetPlatformSettings(bool bInISTVOS, bool bInIsVisionOS);

	/**
	 * Destructor.
	 */
	~FIOSTargetPlatformSettings();

public:
		
	virtual bool SupportsFeature( ETargetPlatformFeatures Feature ) const override;

	virtual void GetAllPossibleShaderFormats( TArray<FName>& OutFormats ) const override;

	virtual void GetAllTargetedShaderFormats( TArray<FName>& OutFormats ) const override;

#if WITH_ENGINE
	virtual void GetReflectionCaptureFormats( TArray<FName>& OutFormats ) const override;

	virtual const class FStaticMeshLODSettings& GetStaticMeshLODSettings( ) const override
	{
		return StaticMeshLODSettings;
	}

	virtual const UTextureLODSettings& GetTextureLODSettings() const override;

	virtual void RegisterTextureLODSettings(const UTextureLODSettings* InTextureLODSettings) override
	{
		TextureLODSettings = InTextureLODSettings;
	}

#endif // WITH_ENGINE

	//~ Begin ITargetPlatform Interface

	virtual bool UsesDistanceFields() const override;
	
	virtual bool UsesRayTracing() const override;
	virtual ERayTracingRuntimeMode GetRayTracingMode() const override;

private:
	
	// true if this is targeting TVOS vs IOS
	bool bIsTVOS;
	bool bIsVisionOS;
#if WITH_ENGINE
	// Holds the cache of the target LOD settings.
	const UTextureLODSettings* TextureLODSettings;

	// Holds the static mesh LOD settings.
	FStaticMeshLODSettings StaticMeshLODSettings;
#endif // WITH_ENGINE
	bool GetIsDistanceField() const;
	int32 GetMobileShadingPath() const ;
	bool GetIsMobileForwardEnableClusteredReflections() const;
};
