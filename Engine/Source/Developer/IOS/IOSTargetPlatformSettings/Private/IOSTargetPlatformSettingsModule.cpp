// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSTargetPlatformSettings.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Modules/ModuleManager.h"

/**
 * Module for iOS as a target platform settings
 */
class FIOSTargetPlatformSettingsModule : public ITargetPlatformSettingsModule
{
public:
	virtual void GetTargetPlatformSettings(TArray<ITargetPlatformSettings*>& TargetPlatforms) override
	{
		TargetPlatforms.Add(new FIOSTargetPlatformSettings(false, false));
	}
};


IMPLEMENT_MODULE(FIOSTargetPlatformSettingsModule, IOSTargetPlatformSettings);
