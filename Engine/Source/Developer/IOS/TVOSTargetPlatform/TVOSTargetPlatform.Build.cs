// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TVOSTargetPlatform : ModuleRules
{
	public TVOSTargetPlatform(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "IOS";
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.TVOS);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"TargetPlatform",
				"DesktopPlatform",
			}
		);
	}
}
