// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSTargetPlatformControls.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Modules/ModuleManager.h"

/**
 * Module for TVOS as a target platform controls
 */
class FTVOSTargetPlatformControlsModule	: public ITargetPlatformControlsModule
{
public:
	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName) override
	{
		if (FIOSTargetPlatformControls::IsUsable())
		{
			ITargetPlatformSettingsModule* ModuleSettings = FModuleManager::GetModulePtr<ITargetPlatformSettingsModule>(PlatformSettingsModuleName);
			if (ModuleSettings != nullptr)
			{
				TArray<ITargetPlatformSettings*> TargetPlatformSettings = ModuleSettings->GetTargetPlatformSettings();
				check(TargetPlatformSettings.Num() == 1);

				TargetPlatforms.Add(new FIOSTargetPlatformControls(true, false, false, TargetPlatformSettings[0]));
				TargetPlatforms.Add(new FIOSTargetPlatformControls(true, false, true, TargetPlatformSettings[0]));
			}
		}
	}
};


IMPLEMENT_MODULE(FTVOSTargetPlatformControlsModule, TVOSTargetPlatformControls);
