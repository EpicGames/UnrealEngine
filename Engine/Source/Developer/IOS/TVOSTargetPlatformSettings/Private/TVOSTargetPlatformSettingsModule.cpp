// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSTargetPlatformSettings.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Modules/ModuleManager.h"

/**
 * Module for TVOS as a target platform settings
 */
class FTVOSTargetPlatformSettingsModule	: public ITargetPlatformSettingsModule
{
public:

	virtual void GetTargetPlatformSettings(TArray<ITargetPlatformSettings*>& TargetPlatforms) override
	{
		TargetPlatforms.Add(new FIOSTargetPlatformSettings(true, false));
	}
};


IMPLEMENT_MODULE(FTVOSTargetPlatformSettingsModule, TVOSTargetPlatformSettings);
